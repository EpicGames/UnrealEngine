{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Concert Sync Core",
	"Description": "Shared plugin for Concert Sync client and server plugins",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor", "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "ConcertSyncCore",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault",
			"ProgramAllowList": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor", "LiveLinkHub" ]
		}
	],
	"Plugins": [
		{
			"Name": "ConcertMain",
			"Enabled": true,
			"TargetDenyList": [
				"Client",
				"Server"
			]
		},
		{
			"Name": "SQLiteCore",
			"Enabled": true,
			"TargetDenyList": [
				"Client",
				"Server"
			]
		}
	]
}
