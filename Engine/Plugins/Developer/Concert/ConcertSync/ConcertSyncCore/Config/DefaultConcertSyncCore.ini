[/Script/ConcertSyncCore.ConcertSyncConfig]
+IncludeObjectClassFilters=(ObjectOuterClass=None,ObjectClasses=(Class'/Script/Engine.World',Class'/Script/Engine.BookmarkBase',Class'/Script/Engine.MaterialParameterCollection',Class'/Script/MovieScene.MovieSceneSignedObject',Class'/Script/MovieScene.MovieSceneFolder'))
+IncludeObjectClassFilters=(ObjectOuterClass=Class'/Script/Engine.World',ObjectClasses=(Class'/Script/Engine.Layer',Class'/Script/Engine.Level',Class'/Script/Engine.LevelStreaming',Class'/Script/Engine.Actor',Class'/Script/Engine.ActorComponent',Class'/Script/Engine.Polys',Class'/Script/Engine.BrushBuilder',Class'/Script/Foliage.FoliageType'))
+IncludeObjectClassFilters=(ObjectOuterClass=Class'/Script/MovieScene.MovieSceneSignedObject',ObjectClasses=(Class'/Script/Engine.Actor',Class'/Script/Engine.ActorComponent'))
+IncludeObjectClassFilters=(ObjectOuterClass=Class'/Script/Engine.World',ObjectClasses=(Class'/Script/Engine.MaterialInstanceDynamic', '/Script/Engine.MaterialInstanceConstant', '/Script/MediaAssets.MediaPlaylist'))
+IncludeObjectClassFilters=(ObjectOuterClass=Class'/Script/Engine.World',ObjectClasses=(Class'/Script/Constraints.TickableConstraint', '/Script/Constraints.TransformableHandle', '/Script/Constraints.ConstraintsActor', '/Script/Constraints.ConstraintsManager'))

+AllowedTransientProperties="/Script/Engine.Actor:bHiddenEdTemporary"
+AllowedTransientProperties="/Script/Engine.Actor:bHiddenEdLevel"
+AllowedTransientProperties="/Script/Engine.Level:bIsVisible"
+AllowedTransientProperties="/Script/Engine.SceneComponent:AttachChildren"
+ExcludedProperties="/Script/Engine.InstanceCacheDataBase:UniqueTransientPackage"
+ExcludeTransactionClassFilters=(ObjectOuterClass=None,ObjectClasses=(Class'/Script/Landscape.LandscapeProxy',Class'/Script/Landscape.LandscapeHeightfieldCollisionComponent',Class'/Script/Landscape.LandscapeComponent'))

+PostReplicateActions=(Action=/Script/ConcertSyncCore.ConcertReplicationAction_UpdateComponentToWorld(),Properties=("/Script/Engine.SceneComponent:RelativeLocation","/Script/Engine.SceneComponent:RelativeRotation","/Script/Engine.SceneComponent:RelativeScale3D","/Script/Engine.SceneComponent:bAbsoluteLocation","/Script/Engine.SceneComponent:bAbsoluteRotation","/Script/Engine.SceneComponent:bAbsoluteScale"))
+PostReplicateActions=(Action=/Script/ConcertSyncCore.ConcertReplicationAction_MarkRenderStateDirty(),Properties=("/Script/Engine.SceneComponent:bVisible"))
