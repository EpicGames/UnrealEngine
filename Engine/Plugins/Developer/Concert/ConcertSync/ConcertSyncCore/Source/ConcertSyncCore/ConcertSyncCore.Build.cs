// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ConcertSyncCore : ModuleRules
	{
		public ConcertSyncCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Concert",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ConcertTransport",
					"Json",
					"SQLiteCore",
					"TraceLog"
				}
			);

			if (Target.bCompileAgainstEngine)
			{
				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"Engine",
					}
				);
			}
		}
	}
}
