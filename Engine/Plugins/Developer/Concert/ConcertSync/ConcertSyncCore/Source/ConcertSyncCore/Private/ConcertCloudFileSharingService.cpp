// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertCloudFileSharingService.h"

#include "HAL/FileManager.h"
#include "ConcertUtil.h"
#include "Misc/Guid.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "HAL/PlatformProcess.h"

#include "ConcertLogGlobal.h"

FConcertCloudFileSharingService::FConcertCloudFileSharingService(const FString& Role)
{
}

FConcertCloudFileSharingService::~FConcertCloudFileSharingService()
{
}

IConcertFileSharingService* FConcertCloudFileSharingService::Clone() const
{
	return new FConcertCloudFileSharingService(*this);
}

bool FConcertCloudFileSharingService::IsEnabled() const
{
	return bEnabled && PathIsValid();
}

void FConcertCloudFileSharingService::SetEnabled(bool bInEnabled)
{
	bEnabled = bInEnabled;
}

bool FConcertCloudFileSharingService::PathIsValid() const
{
	FFileStatData StatData = IPlatformFile::GetPlatformPhysical().GetStatData(*SharedRootPathname);
	return !StatData.bIsReadOnly && StatData.bIsValid && StatData.bIsDirectory;
}

void FConcertCloudFileSharingService::SetPath(const FString& InPath)
{
	FString Path = FPaths::ConvertRelativePathToFull(InPath);
	FFileStatData StatData = IPlatformFile::GetPlatformPhysical().GetStatData(*Path);
	if (!StatData.bIsReadOnly)
	{
		if (!IFileManager::Get().DirectoryExists(*Path))
		{
			IPlatformFile::GetPlatformPhysical().CreateDirectoryTree(*Path);
		}
	}

	// Even though we may not have a valid path, we still store the path name for use.
	SharedRootPathname = Path; 

	if (!PathIsValid())
	{
		UE_LOGF(LogConcert, Display, "ConcertCloudFileSharingService - Unable to set path to %ls. File sharing will be disabled.", *Path);
	}
}

FString FConcertCloudFileSharingService::GetPath() const
{
	return SharedRootPathname;
}

void FConcertCloudFileSharingService::Delete(const FString& InFileOrDirectory)
{
	FString FileOrDirectory = FPaths::ConvertRelativePathToFull(InFileOrDirectory);
	if (!(FPaths::IsUnderDirectory(FileOrDirectory, SharedRootPathname) || FPaths::IsSamePath(FileOrDirectory, SharedRootPathname)))
	{
		return;
	}

	FFileStatData StatData = IPlatformFile::GetPlatformPhysical().GetStatData(*FileOrDirectory);
	if (StatData.bIsValid && StatData.bIsDirectory)
	{
		IPlatformFile::GetPlatformPhysical().DeleteDirectoryRecursively(*FileOrDirectory);
	}
	else if (StatData.bIsValid)
	{
		IFileManager::Get().Delete(*FileOrDirectory);
	}
}

bool FConcertCloudFileSharingService::Publish(const FString& Pathname, FString& OutFileId)
{
	// The file share must be enabled to allow for publishing.
	if (!IsEnabled())
	{
		UE_LOGF(LogConcert, Error, "ConcertCloudFileSharingService - Publishing %ls is not allowed because sharing.", *Pathname);
		return false;
	}

	FString PackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(Pathname, PackageName))
	{
		FString Path = FPackageName::GetLongPackagePath(PackageName);
		FString OutPath = SharedRootPathname / Path;
		if (!IFileManager::Get().DirectoryExists(*OutPath))
		{
			IPlatformFile::GetPlatformPhysical().CreateDirectoryTree(*OutPath);
		}

		const bool bIncludeDot = true;
		FString Extension = FPaths::GetExtension( Pathname, true );

		// Copy File will fail if it already exists. If the user saves a new version of an asset, we simply append a version number to
		// keep uniqueness.
		FString PathNoExt = SharedRootPathname / PackageName;
		FString CandidatePath = PathNoExt + Extension;
		int32 VersionIdx = 2;
		while (IFileManager::Get().FileExists(*CandidatePath))
		{
			CandidatePath = PathNoExt + TEXT("_v") + FString::FromInt(VersionIdx) + Extension;
			VersionIdx++;
		}

		OutFileId = CandidatePath;
	}
	else
	{
		OutFileId = SharedRootPathname / FGuid::NewGuid().ToString();
	}
		
	// Create a unique name for the file.
	UE_LOGF(LogConcert, Display, "ConcertCloudFileSharingService - Publishing %ls to %ls.", *Pathname, *OutFileId);
	return IFileManager::Get().Copy(*OutFileId, *Pathname) == COPY_OK;
}

bool FConcertCloudFileSharingService::Publish(FArchive& SrcAr, int64 Size, FString& OutFileId)
{
	// The file share must be enabled tollow for publishing.
	if (!IsEnabled())
	{
		UE_LOGF(LogConcert, Error, "ConcertCloudFileSharingService - Publishing archive is not allowed because sharing is not enabled.");
		return false;
	}

	// Create a unique name for the file.
	FString SharedFilePathname = SharedRootPathname / FGuid::NewGuid().ToString();
	TUniquePtr<FArchive> SharedFileWriter(IFileManager::Get().CreateFileWriter(*SharedFilePathname, FILEWRITE_AllowRead));
	if (SharedFileWriter)
	{
		ConcertUtil::Copy(*SharedFileWriter, SrcAr, Size);
		OutFileId = FPaths::ConvertRelativePathToFull(SharedFilePathname);
		UE_LOGF(LogConcert, Display, "ConcertCloudFileSharingService - Publishing Archive to %ls.", *OutFileId);
		return true;
	}
	return false;
}

TSharedPtr<FArchive> FConcertCloudFileSharingService::CreateReader(const FString& InFileId)
{
	FString FullInFileId = FPaths::ConvertRelativePathToFull(InFileId);
	if (FPaths::IsUnderDirectory(FullInFileId, SharedRootPathname) && IFileManager::Get().FileExists(*FullInFileId))
	{
		UE_LOGF(LogConcert, Verbose, "ConcertCloudFileSharingService - Creating reader for %ls.", *FullInFileId);
		return MakeShareable(IFileManager::Get().CreateFileReader(*FullInFileId));
	}

	return nullptr; // File not found.
}

TSharedPtr<FArchive> FConcertCloudFileSharingService::CreateWriter(FString& OutFileId)
{
	// Users are allow to call this function for generating challenge files. These files can be
	if (PathIsValid())
	{
		OutFileId = SharedRootPathname / FGuid::NewGuid().ToString();
		return MakeShareable(IFileManager::Get().CreateFileWriter(*OutFileId, FILEWRITE_AllowRead));
	}

	return nullptr;
}
