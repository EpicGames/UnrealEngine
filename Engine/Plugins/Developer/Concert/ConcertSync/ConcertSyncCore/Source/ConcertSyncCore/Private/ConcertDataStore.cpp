// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertDataStore.h"
#include "ConcertLogGlobal.h"

FConcertDataStoreResult FConcertDataStore::FetchOrAdd(const FName& Key, const FName& TypeName, const FConcertSessionSerializedPayload& SerializedValue)
{
	SCOPED_CONCERT_TRACE(FConcertDataStore_FetchOrAdd);
	check(!Key.IsNone() && !TypeName.IsNone());
	check(SerializedValue.PayloadSize > 0);

	if (FConcertDataStoreValueRef* StoredValue = KeyValueMap.Find(Key))
	{
		if (TypeName == (*StoredValue)->TypeName)
		{
			return FConcertDataStoreResult(EConcertDataStoreResultCode::Fetched, FConcertDataStoreValueConstPtr(*StoredValue));
		}
		else
		{
			return FConcertDataStoreResult(EConcertDataStoreResultCode::TypeMismatch);
		}
	}
	else
	{
		uint32 Version = 1;
		return InternalAdd(Key, TypeName, SerializedValue, Version);
	}
}

FConcertDataStoreResult FConcertDataStore::Store(const FName& Key, const FName& TypeName, const FConcertSessionSerializedPayload& SerializedValue, const TOptional<uint32>& Version)
{
	SCOPED_CONCERT_TRACE(FConcertDataStore_Store);
	check(!Key.IsNone() && !TypeName.IsNone());
	check(SerializedValue.PayloadSize > 0);

	if (FConcertDataStoreValueRef* StoredValue = KeyValueMap.Find(Key))
	{
		if (TypeName == (*StoredValue)->TypeName)
		{
			// If the update policy is to overwrite or no client has a reference on the current value, in this
			// case we are allowed to perform an in-place update, no client will ever gets its value(s) swapped implicitly.
			if (UpdatePolicy == EUpdatePolicy::Overwrite || (*StoredValue).IsUnique())
			{
				(*StoredValue)->Version = (Version.IsSet() ? Version.GetValue() : (*StoredValue)->Version + 1);
				(*StoredValue)->SerializedValue = SerializedValue;
			}
			else // Replace the shared pointer to ensure client(s) holding the value will not notice the update (Just as if they had a copy).
			{
				*StoredValue = MakeShared<FConcertDataStore_StoreValue, ESPMode::ThreadSafe>(FConcertDataStore_StoreValue{TypeName, Version.IsSet() ? Version.GetValue() : (*StoredValue)->Version + 1, SerializedValue});
			}

			return FConcertDataStoreResult(EConcertDataStoreResultCode::Exchanged, FConcertDataStoreValueConstPtr(*StoredValue));
		}
		else
		{
			return FConcertDataStoreResult(EConcertDataStoreResultCode::TypeMismatch);
		}
	}
	else
	{
		return InternalAdd(Key, TypeName, SerializedValue, Version.IsSet() ? Version.GetValue() : 1);
	}
}

FConcertDataStoreResult FConcertDataStore::Fetch(const FName& Key, const FName& TypeName) const
{
	SCOPED_CONCERT_TRACE(FConcertDataStore_Fetch);
	check(!Key.IsNone() && !TypeName.IsNone());

	if (const FConcertDataStoreValueRef* StoredValue = KeyValueMap.Find(Key))
	{
		if (TypeName == (*StoredValue)->TypeName)
		{
			return FConcertDataStoreResult(EConcertDataStoreResultCode::Fetched, FConcertDataStoreValueConstPtr(*StoredValue));
		}
		else
		{
			return FConcertDataStoreResult(EConcertDataStoreResultCode::TypeMismatch);
		}
	}

	return FConcertDataStoreResult(EConcertDataStoreResultCode::NotFound);
}

FConcertDataStoreResult FConcertDataStore::InternalAdd(const FName& Key, const FName& TypeName, const FConcertSessionSerializedPayload& Value, uint32 Version)
{
	check(KeyValueMap.Find(Key) == nullptr);
	return FConcertDataStoreResult(EConcertDataStoreResultCode::Added, FConcertDataStoreValueConstPtr(KeyValueMap.Add(Key, MakeShared<FConcertDataStore_StoreValue, ESPMode::ThreadSafe>(FConcertDataStore_StoreValue{TypeName, Version, Value}))));
}

TOptional<uint32> FConcertDataStore::GetVersion(const FName& Key) const
{
	TOptional<uint32> Version;

	if (const FConcertDataStoreValueRef* Value = KeyValueMap.Find(Key))
	{
		Version.Emplace((*Value)->Version);
	}
	return Version;
}

void FConcertDataStore::Visit(const TFunctionRef<void(const FName&, const FConcertDataStore_StoreValue&)>& Visitor) const
{
	for (const auto& Element : KeyValueMap)
	{
		Visitor(Element.Key, *Element.Value);
	}
}
