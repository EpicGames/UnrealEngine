// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertFileCache.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

namespace ConcertFileCacheUtil
{

FString GetInternalFilename(const FString& InFilename)
{
	return FPaths::ConvertRelativePathToFull(InFilename);
}

FDateTime GetInternalTimestamp(const FString& InFilename)
{
	return IFileManager::Get().GetTimeStamp(*InFilename);
}

bool IsCachedFileValid(const FString& InFilename, const FDateTime& InCachedTimestamp)
{
	return GetInternalTimestamp(InFilename) == InCachedTimestamp;
}

}

FConcertFileCache::FConcertFileCache(const int32 InMinimumNumberOfFilesToCache, const uint64 InMaximumNumberOfBytesToCache)
	: MinimumNumberOfFilesToCache(InMinimumNumberOfFilesToCache)
	, MaximumNumberOfBytesToCache(InMaximumNumberOfBytesToCache)
	, TotalCachedFileDataBytes(0)
	, InternalCache(1000)
{
	check(MinimumNumberOfFilesToCache >= 0);
}

FConcertFileCache::~FConcertFileCache()
{
	InternalCache.Empty();
	checkf(TotalCachedFileDataBytes == 0, TEXT("File Cache leaked %lld bytes during tracking!"), TotalCachedFileDataBytes);
}

bool FConcertFileCache::CacheFile(const FString& InFilename)
{
	const FString InternalFilename = ConcertFileCacheUtil::GetInternalFilename(InFilename);

	TArray<uint8> FileData;
	if (FFileHelper::LoadFileToArray(FileData, *InternalFilename))
	{
		TSharedPtr<FInternalCacheEntry> InternalCacheEntry = InternalCache.FindAndTouchRef(InternalFilename);
		if (InternalCacheEntry.IsValid())
		{
			InternalCacheEntry->SetFile(MoveTemp(FileData), ConcertFileCacheUtil::GetInternalTimestamp(InternalFilename));
		}
		else
		{
			InternalCache.Add(InternalFilename, MakeShared<FInternalCacheEntry>(MoveTemp(FileData), ConcertFileCacheUtil::GetInternalTimestamp(InternalFilename), TotalCachedFileDataBytes));
		}

		TrimCache();
		return true;
	}
	
	InternalCache.Remove(InternalFilename);
	return false;
}

bool FConcertFileCache::SaveAndCacheFile(const FString& InFilename, TArray<uint8>&& InFileData)
{
	const FString InternalFilename = ConcertFileCacheUtil::GetInternalFilename(InFilename);

	if (FFileHelper::SaveArrayToFile(InFileData, *InternalFilename))
	{
		TSharedPtr<FInternalCacheEntry> InternalCacheEntry = InternalCache.FindAndTouchRef(InternalFilename);
		if (InternalCacheEntry.IsValid())
		{
			InternalCacheEntry->SetFile(MoveTemp(InFileData), ConcertFileCacheUtil::GetInternalTimestamp(InternalFilename));
		}
		else
		{
			InternalCache.Add(InternalFilename, MakeShared<FInternalCacheEntry>(MoveTemp(InFileData), ConcertFileCacheUtil::GetInternalTimestamp(InternalFilename), TotalCachedFileDataBytes));
		}

		TrimCache();
		return true;
	}

	InternalCache.Remove(InternalFilename);
	return false;
}

void FConcertFileCache::UncacheFile(const FString& InFilename)
{
	const FString InternalFilename = ConcertFileCacheUtil::GetInternalFilename(InFilename);
	InternalCache.Remove(InternalFilename);
}

bool FConcertFileCache::FindOrCacheFile(const FString& InFilename, TArray<uint8>& OutFileData)
{
	const FString InternalFilename = ConcertFileCacheUtil::GetInternalFilename(InFilename);
	OutFileData.Reset();

	TSharedPtr<FInternalCacheEntry> InternalCacheEntry = InternalCache.FindAndTouchRef(InternalFilename);
	if (InternalCacheEntry.IsValid() && ConcertFileCacheUtil::IsCachedFileValid(InternalFilename, InternalCacheEntry->GetFileTimestamp()))
	{
		OutFileData = InternalCacheEntry->GetFileData();
		return true;
	}

	if (FFileHelper::LoadFileToArray(OutFileData, *InternalFilename))
	{
		if (InternalCacheEntry.IsValid())
		{
			InternalCacheEntry->SetFile(CopyTemp(OutFileData), ConcertFileCacheUtil::GetInternalTimestamp(InternalFilename));
		}
		else
		{
			InternalCache.Add(InternalFilename, MakeShared<FInternalCacheEntry>(CopyTemp(OutFileData), ConcertFileCacheUtil::GetInternalTimestamp(InternalFilename), TotalCachedFileDataBytes));
		}

		TrimCache();
		return true;
	}

	InternalCache.Remove(InternalFilename);
	return false;
}

bool FConcertFileCache::FindFile(const FString& InFilename, TArray<uint8>& OutFileData) const
{
	const FString InternalFilename = ConcertFileCacheUtil::GetInternalFilename(InFilename);
	OutFileData.Reset();

	TSharedPtr<FInternalCacheEntry> InternalCacheEntry = InternalCache.FindRef(InternalFilename);
	if (InternalCacheEntry.IsValid() && ConcertFileCacheUtil::IsCachedFileValid(InternalFilename, InternalCacheEntry->GetFileTimestamp()))
	{
		OutFileData = InternalCacheEntry->GetFileData();
		return true;
	}

	return false;
}

bool FConcertFileCache::HasCachedFile(const FString& InFilename) const
{
	const FString InternalFilename = ConcertFileCacheUtil::GetInternalFilename(InFilename);
	TSharedPtr<FInternalCacheEntry> InternalCacheEntry = InternalCache.FindRef(InternalFilename);
	return InternalCacheEntry.IsValid() && ConcertFileCacheUtil::IsCachedFileValid(InternalFilename, InternalCacheEntry->GetFileTimestamp());
}

void FConcertFileCache::TrimCache()
{
	while (TotalCachedFileDataBytes > MaximumNumberOfBytesToCache && InternalCache.Num() > MinimumNumberOfFilesToCache)
	{
		if (!InternalCache.RemoveLeastRecent())
		{
			break;
		}
	}
}
