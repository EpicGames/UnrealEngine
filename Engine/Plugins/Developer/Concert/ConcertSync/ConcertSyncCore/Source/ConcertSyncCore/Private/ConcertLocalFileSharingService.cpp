// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLocalFileSharingService.h"
#include "HAL/FileManager.h"
#include "ConcertUtil.h"
#include "Misc/Guid.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/Paths.h"
#include "HAL/PlatformProcess.h"

#include "ConcertLogGlobal.h"

FConcertLocalFileSharingService::FConcertLocalFileSharingService(const FString& Role)
	: SharedRootPathname(FPaths::EngineIntermediateDir() / Role / TEXT("TempFileShare")) // The EngineIntermediateDir is expected to be know of both the client and the server.
	, SystemMutexName(TEXT("ConcertFileSharingMutex_777221B1ZZ")) // Arbitrary name, but unique enough to avoid collision with other global mutexes from other apps.
	, ActiveServicesRepositoryPathname(SharedRootPathname / TEXT("ActiveProcesses.json"))
{
	FSystemWideCriticalSection ScopedSysWideMutex(SystemMutexName);

	TArray<uint32> ActivesPids;
	LoadActiveServices(ActivesPids);
	RemoveDeadProcessesAndFiles(ActivesPids);
	ActivesPids.Add(FPlatformProcess::GetCurrentProcessId());
	SaveActiveServices(ActivesPids);
}

FConcertLocalFileSharingService::~FConcertLocalFileSharingService()
{
	FSystemWideCriticalSection ScopedSysWideMutex(SystemMutexName);

	TArray<uint32> ActivePids;
	LoadActiveServices(ActivePids);
	ActivePids.Remove(FPlatformProcess::GetCurrentProcessId());
	RemoveDeadProcessesAndFiles(ActivePids);
	SaveActiveServices(ActivePids);
}

IConcertFileSharingService* FConcertLocalFileSharingService::Clone() const
{
	return new FConcertLocalFileSharingService(*this);
}

bool FConcertLocalFileSharingService::IsEnabled() const
{
	// If the user has specified a local file sharing service then we are always enabled.  
	return true;
}

void FConcertLocalFileSharingService::SetEnabled(bool bInEnabled)
{
	// We do not support disabling this service. If the user has specified a local file service then we always use it.
}

void FConcertLocalFileSharingService::SetPath(const FString& Path)
{
	// We do not support setting althernative paths with this file service. 
}

FString FConcertLocalFileSharingService::GetPath() const
{
	return SharedRootPathname;
}

void FConcertLocalFileSharingService::Delete(const FString& OutFileId)
{
	// This class does not handle the delete method.
	UE_LOGF(LogConcert, Warning, "LocalFileSharingService - Delete is not supported with this file sharing service. Leaving %ls inplace.", *OutFileId);
}

void FConcertLocalFileSharingService::RemoveDeadProcessesAndFiles(TArray<uint32>& InOutPids)
{
	// Remove all dead process.
	InOutPids.RemoveAll([](uint32 Pid)
	{
		return !FPlatformProcess::IsApplicationRunning(Pid);
	});

	// If no processes are left using the shared directory, delete expired files. Normally, files are deleted once consumed, but in case of a crash, some files may be left over.
	if (InOutPids.Num() == 0)
	{
		IFileManager::Get().IterateDirectory(*SharedRootPathname, [this](const TCHAR* Pathname, bool bIsDirectory)
		{
			// The delay is a security in case the file containing the active service PID failed to open or another process failed to updated it, so the list of PID provided above may be incomplete.
			// The shared files are expected to be consumed pretty much immediatedly. So adding a 1h expiration delay sounds reasonable.
			constexpr double FileExpirationDelaySeconds = 3600;
			if (bIsDirectory || ActiveServicesRepositoryPathname == Pathname || IFileManager::Get().GetFileAgeSeconds(Pathname) < FileExpirationDelaySeconds)
			{
				return true; // Continue -> shared files are stored flat, don't delete the json file containing the pid and keep files for at least 1h.
			}

			IFileManager::Get().Delete(Pathname, /*bRequireExit*/false, /*bEventIfReadOnly*/false, /*bQuiet*/true);
			return true;
		});
	}
}

bool FConcertLocalFileSharingService::Publish(const FString& Pathname, FString& OutFileId)
{
	// Create a unique name for the file.
	OutFileId = SharedRootPathname / FGuid::NewGuid().ToString();
	return IFileManager::Get().Copy(*OutFileId, *Pathname) == COPY_OK;
}

bool FConcertLocalFileSharingService::Publish(FArchive& SrcAr, int64 Size, FString& OutFileId)
{
	// Create a unique name for the file.
	FString SharedFilePathname = SharedRootPathname / FGuid::NewGuid().ToString();
	TUniquePtr<FArchive> SharedFileWriter(IFileManager::Get().CreateFileWriter(*SharedFilePathname));
	if (SharedFileWriter)
	{
		ConcertUtil::Copy(*SharedFileWriter, SrcAr, Size);
		OutFileId = FPaths::ConvertRelativePathToFull(SharedFilePathname);
		return true;
	}
	return false;
}

TSharedPtr<FArchive> FConcertLocalFileSharingService::CreateReader(const FString& InFileId)
{
	if (IFileManager::Get().FileExists(*InFileId))
	{
		// Delete the files when the archive goes out of scope, so that consumed files don't use too much space.
		auto Deleter = [InFileId](FArchive* Ar)
		{
			delete Ar;
			IFileManager::Get().Delete(*InFileId);
		};

		return MakeShareable(IFileManager::Get().CreateFileReader(*InFileId), MoveTemp(Deleter));
	}

	return nullptr; // File not found.
}

TSharedPtr<FArchive> FConcertLocalFileSharingService::CreateWriter(FString& OutFileId)
{
	OutFileId = SharedRootPathname / FGuid::NewGuid().ToString();
	return MakeShareable(IFileManager::Get().CreateFileWriter(*OutFileId, FILEWRITE_AllowRead));
}

void FConcertLocalFileSharingService::LoadActiveServices(TArray<uint32>& OutPids)
{
	if (IFileManager::Get().FileExists(*ActiveServicesRepositoryPathname))
	{
		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*ActiveServicesRepositoryPathname));
		if (Ar)
		{
			TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Ar.Get());

			TArray< TSharedPtr<FJsonValue> > Pids;
			if (FJsonSerializer::Deserialize(JsonReader, Pids))
			{
				for (const TSharedPtr<FJsonValue>& PidVal : Pids)
				{
					uint32 Pid;
					if (PidVal->TryGetNumber(Pid))
					{
						OutPids.Add(Pid);
					}
				}
			}
		}
	}
}

void FConcertLocalFileSharingService::SaveActiveServices(const TArray<uint32>& InPids)
{
	if (IFileManager::Get().DirectoryExists(*SharedRootPathname) || IFileManager::Get().MakeDirectory(*SharedRootPathname, /*Tree*/true))
	{
		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*ActiveServicesRepositoryPathname));
		if (Ar)
		{
			TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(Ar.Get());

			JsonWriter->WriteArrayStart();
			for (uint32 Pid : InPids)
			{
				JsonWriter->WriteValue(static_cast<int64>(Pid));
			}
			JsonWriter->WriteArrayEnd();
		}
	}
}
