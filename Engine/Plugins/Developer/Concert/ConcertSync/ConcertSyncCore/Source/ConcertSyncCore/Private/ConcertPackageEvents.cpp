// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertPackageEvents.h"

namespace UE::ConcertSyncCore::ConcertPackageEvents
{
	FConcertBeginSendPackageDelegate& OnLocalBeginSendPackage()
	{
		static FConcertBeginSendPackageDelegate Instance;
		return Instance;
	}
	
	FConcertFinishSendPackageDelegate& OnLocalFinishSendPackage()
	{
		static FConcertFinishSendPackageDelegate Instance;
		return Instance;
	}

	FConcertBeginSendPackageDelegate& OnRemoteBeginSendPackage()
	{
		static FConcertBeginSendPackageDelegate Instance;
		return Instance;
	}

	FConcertFinishSendPackageDelegate& OnRemoteFinishSendPackage()
	{
		static FConcertFinishSendPackageDelegate Instance;
		return Instance;
	}

	FConcertRejectSendPackageDelegate& OnRejectRemoteSendPackage()
	{
		static FConcertRejectSendPackageDelegate Instance;
		return Instance;
	}
}