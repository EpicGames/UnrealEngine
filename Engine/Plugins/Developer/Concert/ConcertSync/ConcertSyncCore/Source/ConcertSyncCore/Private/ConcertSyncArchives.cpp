// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncArchives.h"
#include "ConcertSyncSettings.h"
#include "ConcertVersion.h"

#include "Misc/PackageName.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "UObject/Package.h"

static const FSoftObjectPath SkipAssetsMarker = FSoftObjectPath(TEXT("/Engine/Transient.__CONCERT_SKIP_ASSETS__"));

namespace ConcertSyncUtil
{

const FSoftObjectPath& GetSkipObjectPath()
{
	return SkipAssetsMarker;
}

bool CanExportProperty(const FProperty* Property, const bool InIncludeEditorOnlyData)
{
	auto PropertyPathIsInList = [Property](const TArray<TFieldPath<FProperty>>& PropertyPaths)
	{
		return PropertyPaths.ContainsByPredicate([Property](const TFieldPath<FProperty>& PropertyFieldPath)
		{
			const FProperty* FilterProperty = PropertyFieldPath.Get();
			return Property == FilterProperty;
		});
	};

	auto PropertyTypeIsInList = [Property](const TArray<FName>& PropertyTypes)
	{
		return PropertyTypes.ContainsByPredicate([Property](const FName PropertyType)
		{
			const FFieldClass* FilterPropertyClass = FFieldClass::GetNameToFieldClassMap().FindRef(PropertyType);
			return FilterPropertyClass && Property->GetClass()->IsChildOf(FilterPropertyClass);
		});
	};

	const UConcertSyncConfig* SyncConfig = GetDefault<UConcertSyncConfig>();
	return (!Property->IsEditorOnlyProperty() || InIncludeEditorOnlyData)
		&& (!Property->HasAnyPropertyFlags(CPF_NonTransactional))
		&& (!Property->HasAnyPropertyFlags(CPF_Transient) || PropertyPathIsInList(SyncConfig->AllowedTransientProperties))
		&& (!PropertyPathIsInList(SyncConfig->ExcludedProperties))
		&& (!PropertyTypeIsInList(SyncConfig->ExcludedPropertyTypes));
}

void GatherDefaultSubobjectPaths(const UObject* Obj, TSet<FSoftObjectPath>& OutSubobjects, const bool bOuterIsDSO = false)
{
	ForEachObjectWithOuter(Obj, [bOuterIsDSO, &OutSubobjects](UObject* InnerObj)
	{
		if (bOuterIsDSO || InnerObj->HasAnyFlags(RF_DefaultSubObject) || InnerObj->IsDefaultSubobject())
		{
			OutSubobjects.Emplace(InnerObj);
			GatherDefaultSubobjectPaths(InnerObj, OutSubobjects, /*bOuterIsDSO*/true);
		}
	}, EGetObjectsFlags::None);
}

bool ShouldSkipPropertyDuringReset(const FArchive& Ar, const FProperty* Property, const EPropertyFlags PropertySkipFlags, const FResetValueOptions& Options)
{
	if (Property->HasAnyPropertyFlags(PropertySkipFlags))
	{
		return true;
	}

	// Skip properties that Concert can't edit, as resetting them may destroy bookkeeping data
	{
		// Note: We do this using the root property being serialized (rather than the given property), 
		//       as Concert filters currently only include top-level properties
		const FProperty* RootProperty = Property;
		if (const FArchiveSerializedPropertyChain* PropertyChain = Ar.GetSerializedPropertyChain();
			PropertyChain && PropertyChain->GetNumProperties() > 0)
		{
			RootProperty = PropertyChain->GetPropertyFromRoot(0);
		}
		if (!CanExportProperty(RootProperty, Options.bIncludeEditorOnlyData))
		{
			return true;
		}
	}

	if (Options.ShouldSkipProperty && Options.ShouldSkipProperty(Ar, Property))
	{
		return true;
	}

	return false;
}

void ResetObjectPropertiesToArchetypeValues(UObject* Object, const bool InIncludeEditorOnlyData)
{
	if (const UObject* ObjectArchetype = Object->GetArchetype())
	{
		FResetValueOptions Options;
		Options.bIncludeEditorOnlyData = InIncludeEditorOnlyData;
		Options.bIsDuplicate = false;
		Options.ObjectsToSkip.Add(Object);
		Options.ObjectsToSkip.Add(ObjectArchetype);
		GatherDefaultSubobjectPaths(Object, Options.ObjectsToSkip);
		GatherDefaultSubobjectPaths(ObjectArchetype, Options.ObjectsToSkip);

		ResetObjectPropertiesToTemplateValues(Object, ObjectArchetype, Options);
	}
}

void ResetObjectPropertiesToTemplateValues(UObject* Object, const UObject* Template, const FResetValueOptions& Options)
{
	static constexpr EPropertyFlags InstancedObjectPropertyFlags = CPF_InstancedReference | CPF_PersistentInstance;

	class FTemplatePropertyWriter final : public FObjectWriter
	{
	public:
		FTemplatePropertyWriter(const UObject* Obj, TArray<uint8>& OutBytes, const FResetValueOptions& InOptions)
			: FObjectWriter(OutBytes)
		{
			Options = &InOptions;

			ArIgnoreClassRef = true;
			ArIgnoreArchetypeRef = true;
			ArNoDelta = true;
			ArIsSaveGame = Options->bSaveGameOnly;
			ArPortFlags |= (Options->bIsDuplicate ? PPF_Duplicate : PPF_None);
			ArPropertySkipFlags |= (Options->bIsDuplicate ? CPF_DuplicateTransient : CPF_None);
			ArPropertySkipFlags |= InstancedObjectPropertyFlags; // We can't handle resetting instanced objects

			SetFilterEditorOnly(!Options->bIncludeEditorOnlyData);

			Obj->SerializeScriptProperties(*this);
		}

		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
		{
			return FObjectWriter::ShouldSkipProperty(InProperty)
				|| ShouldSkipPropertyDuringReset(*this, InProperty, ArPropertySkipFlags, *Options);
		}

	private:
		const FResetValueOptions* Options = nullptr;
		EPropertyFlags ArPropertySkipFlags = CPF_None;
	};

	class FTemplatePropertyReader final : public FObjectReader
	{
	public:
		FTemplatePropertyReader(UObject* Obj, const TArray<uint8>& InBytes, const FResetValueOptions& InOptions)
			: FObjectReader(InBytes)
		{
			Options = &InOptions;

			ArIgnoreClassRef = true;
			ArIgnoreArchetypeRef = true;
			ArIsSaveGame = Options->bSaveGameOnly;
			ArPortFlags |= (Options->bIsDuplicate ? PPF_Duplicate : PPF_None);
			ArPropertySkipFlags |= (Options->bIsDuplicate ? CPF_DuplicateTransient : CPF_None);
			ArPropertySkipFlags |= InstancedObjectPropertyFlags; // We can't handle resetting instanced objects

			SetFilterEditorOnly(!Options->bIncludeEditorOnlyData);

#if USE_STABLE_LOCALIZATION_KEYS
			if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
			{
				SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(Obj));
			}
#endif // USE_STABLE_LOCALIZATION_KEYS

			Obj->SerializeScriptProperties(*this);
		}

		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
		{
			return FObjectReader::ShouldSkipProperty(InProperty)
				|| ShouldSkipPropertyDuringReset(*this, InProperty, ArPropertySkipFlags, *Options);
		}

		virtual FArchive& operator<<(UObject*& Value) override
		{
			UObject* Tmp = nullptr;
			FObjectReader::operator<<(Tmp);

			if (Value != Tmp && CanOverwriteObject(Value, Tmp))
			{
				Value = Tmp;
			}

			return *this;
		}

		virtual FArchive& operator<<(FObjectPtr& Value) override
		{
			FObjectPtr Tmp = nullptr;
			FObjectReader::operator<<(Tmp);

			if (Value != Tmp && CanOverwriteObject(Value.Get(), Tmp.Get()))
			{
				Value = Tmp;
			}

			return *this;
		}

		virtual FArchive& operator<<(FLazyObjectPtr& Value) override
		{
			FLazyObjectPtr Tmp;
			FObjectReader::operator<<(Tmp);

			if (Value != Tmp && CanOverwriteObject(Value.Get(), Tmp.Get()))
			{
				Value = Tmp;
			}

			return *this;
		}

		virtual FArchive& operator<<(FSoftObjectPtr& Value) override
		{
			FSoftObjectPtr Tmp;
			FObjectReader::operator<<(Tmp);

			if (Value != Tmp && CanOverwriteObject(Value.ToSoftObjectPath(), Tmp.ToSoftObjectPath()))
			{
				Value = Tmp;
			}

			return *this;
		}

		virtual FArchive& operator<<(FSoftObjectPath& Value) override
		{
			FSoftObjectPath Tmp;
			FObjectReader::operator<<(Tmp);

			if (Value != Tmp && CanOverwriteObject(Value, Tmp))
			{
				Value = Tmp;
			}

			return *this;
		}

		virtual FArchive& operator<<(FWeakObjectPtr& Value) override
		{
			FWeakObjectPtr Tmp;
			FObjectReader::operator<<(Tmp);

			if (Value != Tmp && CanOverwriteObject(Value.Get(), Tmp.Get()))
			{
				Value = Tmp;
			}

			return *this;
		}

	private:
		bool CanOverwriteObject(const FSoftObjectPath& CurObj, const FSoftObjectPath& NewObj) const
		{
			// Archive serialization doesn't catch all CPF_InstancedReference or CPF_PersistentInstance properties via ShouldSkipProperty, 
			// as it's not called in some cases, eg, a TArray<UObject*> will have the FArrayProperty marked as CPF_ContainsInstancedReference 
			// but won't also test the inner FObjectProperty before serializing its value; instead we query the head of the property 
			// serialization stack and verify it isn't an instanced object property before allowing an object replacement to happen
			const FProperty* Prop = GetSerializedProperty();
			return !(Prop && Prop->HasAnyPropertyFlags(InstancedObjectPropertyFlags))
				&& !Options->ObjectsToSkip.Contains(CurObj) 
				&& !Options->ObjectsToSkip.Contains(NewObj)
				&& !(Options->ShouldSkipObjectReference && Options->ShouldSkipObjectReference(*this, CurObj, NewObj));
		}

		const FResetValueOptions* Options = nullptr;
		EPropertyFlags ArPropertySkipFlags = CPF_None;
	};

	TArray<uint8> TemplateData;
	FTemplatePropertyWriter Writer(Template, TemplateData, Options);
	FTemplatePropertyReader Reader(Object, TemplateData, Options);
}

} // namespace ConcertSyncUtil

FString FConcertSyncWorldRemapper::RemapObjectPathName(const FString& InObjectPathName) const
{
	if (RemapDelegate.IsBound())
	{
		FString Result = InObjectPathName;
		RemapDelegate.Execute(Result);
		return MoveTemp(Result);
	}

	return HasMapping() ? InObjectPathName.Replace(*SourceWorldPathName, *DestWorldPathName) : InObjectPathName;
}

bool FConcertSyncWorldRemapper::ObjectBelongsToWorld(const FString& InObjectPathName) const
{
	if (ObjectPathBelongsToWorldDelegate.IsBound() && ObjectPathBelongsToWorldDelegate.Execute(FStringView(InObjectPathName)))
	{
		return true;
	}

	return HasMapping() && (InObjectPathName.StartsWith(SourceWorldPathName) || InObjectPathName.StartsWith(DestWorldPathName));
}

bool FConcertSyncWorldRemapper::HasMapping() const
{
	return RemapDelegate.IsBound() || (SourceWorldPathName.Len() > 0 && DestWorldPathName.Len() > 0);
}

FConcertSyncObjectWriter::FConcertSyncObjectWriter(FConcertLocalIdentifierTable* InLocalIdentifierTable, UObject* InObj, TArray<uint8>& OutBytes, const bool InIncludeEditorOnlyData, const bool InSkipAssets, const FConcertSyncRemapObjectPath& InRemapDelegate)
	: FConcertIdentifierWriter(InLocalIdentifierTable, OutBytes, /*bIsPersistent*/true)
	, bSkipAssets(InSkipAssets)
	, ShouldSkipPropertyFunc()
	, RemapObjectPathDelegate(InRemapDelegate)
{
	ArIgnoreClassRef = false;
	ArIgnoreArchetypeRef = false;
	ArNoDelta = true;
	//SetWantBinaryPropertySerialization(true);

	SetIsTransacting(InIncludeEditorOnlyData);
	SetFilterEditorOnly(!InIncludeEditorOnlyData);

#if USE_STABLE_LOCALIZATION_KEYS
	if (GIsEditor && !(ArPortFlags & PPF_DuplicateForPIE))
	{
		SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(InObj));
	}
#endif // USE_STABLE_LOCALIZATION_KEYS
}

FConcertSyncObjectWriter::FConcertSyncObjectWriter(FConcertLocalIdentifierTable* InLocalIdentifierTable, UObject* InObj, TArray<uint8>& OutBytes, const bool InIncludeEditorOnlyData, const bool InSkipAssets)
	: FConcertSyncObjectWriter(InLocalIdentifierTable, InObj, OutBytes, InIncludeEditorOnlyData, InSkipAssets, FConcertSyncRemapObjectPath())
{
}

void FConcertSyncObjectWriter::SerializeObject(const UObject* InObject, const TArray<const FProperty*>* InPropertiesToWrite, bool bAllowOuters /*= false*/)
{
	PackageName = InObject->GetPackage()->GetPathName();

	if (bSerializeNestedObjects && !bAllowOuters)
	{
		// When serializing nested objects, we need to know when to stop
		// capturing. A property can point to outer or this package and those
		// should not be included in the serialization. So we make note of them
		// here to stop the serialization when those objects are encountered and
		// only store the FSoftObjectPath
		CollectedObjects.Add(InObject->GetPackage()->GetPathName());
		CollectedObjects.Add(InObject->GetPathName());
		UObject* Outer = InObject->GetOuter();
		while (	Outer )
		{
			CollectedObjects.Add(Outer->GetPathName());
			Outer = Outer->GetOuter();
		}
	}

	if (InPropertiesToWrite)
	{
		ShouldSkipPropertyFunc = [InObject, InPropertiesToWrite](const FProperty* InProperty) -> bool
		{
			return !InPropertiesToWrite->Contains(InProperty);
		};

		const_cast<UObject*>(InObject)->Serialize(*this);

		ShouldSkipPropertyFunc = FShouldSkipPropertyFunc();
	}
	else
	{
		const_cast<UObject*>(InObject)->Serialize(*this);
	}
}

void FConcertSyncObjectWriter::SerializeProperty(const FProperty* InProp, const UObject* InObject)
{
	for (int32 Idx = 0; Idx < InProp->ArrayDim; ++Idx)
	{
		InProp->SerializeItem(FStructuredArchiveFromArchive(*this).GetSlot(), InProp->ContainerPtrToValuePtr<void>(const_cast<UObject*>(InObject), Idx));
	}
}

FArchive& FConcertSyncObjectWriter::operator<<(UObject*& Obj)
{
	bool bCanSerializeObject = false;
	FSoftObjectPath ObjPath;
	if (Obj)
	{
		if (bSkipAssets && Obj->IsAsset())
		{
			ObjPath = SkipAssetsMarker;
		}
		else
		{
			FString ObjectPathString = Obj->GetPathName();

			// Check to see if we can serialize before we perform the remapping.
			bCanSerializeObject = !CollectedObjects.Contains(ObjectPathString);

			RemapObjectPathDelegate.ExecuteIfBound(ObjectPathString);
			ObjPath.SetPath(ObjectPathString);
		}
	}
	ObjPath.SerializePath(*this);

	bool bDoSerializeObject = bCanSerializeObject
		&& bSerializeNestedObjects
		&& Obj && Obj->GetPackage()
		&& PackageName == Obj->GetPackage()->GetPathName()
		&& !Obj->IsA<UPackage>();

	if (bSerializeNestedObjects)
	{
		*this << bDoSerializeObject;
	}

	if (bDoSerializeObject)
	{

		// To prevent stack overflow we need to add to the collected objects first.
		CollectedObjects.Add(ObjPath.ToString());

		check(Obj && Obj->GetClass());

		FString ClassName = Obj->GetClass()->GetPathName();
		int32 OutFlags = static_cast<int32>(Obj->GetFlags());
		*this << ClassName;
		*this << OutFlags;

		SerializeObject(Obj, nullptr, true);
	}
	return *this;
}

FArchive& FConcertSyncObjectWriter::operator<<(FLazyObjectPtr& LazyObjectPtr)
{
	UObject* Obj = LazyObjectPtr.Get();
	FUniqueObjectGuid ObjectGuid = LazyObjectPtr.GetUniqueID();
	// Serialize both the object path and the object guid
	*this << Obj;
	*this << ObjectGuid;
	return *this;
}

FArchive& FConcertSyncObjectWriter::operator<<(FObjectPtr& Obj)
{
	UObject* RawObjPtr = Obj.Get();
	return *this << RawObjPtr;
}

FArchive& FConcertSyncObjectWriter::operator<<(FSoftObjectPtr& AssetPtr)
{
	FSoftObjectPath Obj = AssetPtr.ToSoftObjectPath();
	*this << Obj;
	return *this;
}

FArchive& FConcertSyncObjectWriter::operator<<(FSoftObjectPath& AssetPtr)
{
	FSoftObjectPath ObjPath;
	if (bSkipAssets)
	{
		ObjPath = SkipAssetsMarker;
	}
	else
	{
		FString ObjectPathString = AssetPtr.ToString();
		RemapObjectPathDelegate.ExecuteIfBound(ObjectPathString);
		ObjPath.SetPath(ObjectPathString);
	}
	ObjPath.SerializePath(*this);
	return *this;
}

FArchive& FConcertSyncObjectWriter::operator<<(FWeakObjectPtr& Value)
{
	UObject* Obj = Value.Get();
	*this << Obj;
	return *this;
}

FString FConcertSyncObjectWriter::GetArchiveName() const
{
	return TEXT("FConcertSyncObjectWriter");
}

bool FConcertSyncObjectWriter::ShouldSkipProperty(const FProperty* InProperty) const
{
	return (ShouldSkipPropertyFunc && ShouldSkipPropertyFunc(InProperty))
		|| (!ConcertSyncUtil::CanExportProperty(InProperty, !IsFilterEditorOnly()));
}


namespace UE::Concert::Private::ConcertSyncArchiveUtil
{

void InitReaderArchive(FArchive& Ar, const FConcertSessionVersionInfo* InVersionInfo)
{
	Ar.ArIgnoreClassRef = false;
	Ar.ArIgnoreArchetypeRef = false;
	Ar.ArNoDelta = true;
	//Ar.SetWantBinaryPropertySerialization(true);

	if (InVersionInfo)
	{
		FPackageFileVersion UEVersion(InVersionInfo->FileVersion.FileVersion, (EUnrealEngineObjectUE5Version)InVersionInfo->FileVersion.FileVersionUE5);

		Ar.SetUEVer(UEVersion);
		Ar.SetLicenseeUEVer(InVersionInfo->FileVersion.FileVersionLicensee);
		Ar.SetEngineVer(FEngineVersionBase(InVersionInfo->EngineVersion.Major, InVersionInfo->EngineVersion.Minor, InVersionInfo->EngineVersion.Patch, InVersionInfo->EngineVersion.Changelist));

		FCustomVersionContainer EngineCustomVersions;
		for (const FConcertCustomVersionInfo& CustomVersion : InVersionInfo->CustomVersions)
		{
			EngineCustomVersions.SetVersion(CustomVersion.Key, CustomVersion.Version, CustomVersion.FriendlyName);
		}
		Ar.SetCustomVersions(EngineCustomVersions);
	}

	// This is conditional on WITH_EDITORONLY_DATA because the transaction flag is ignored in builds without it and IsTransacting always returns false. So
	// the writer should only be sending non-editor only properties if the reader does not use editor only data.
	Ar.SetIsTransacting(WITH_EDITORONLY_DATA);
	Ar.SetFilterEditorOnly(!WITH_EDITORONLY_DATA);
}

} // namespace UE::Concert::Private::ConcertSyncArchiveUtil

FConcertSyncObjectReader::FConcertSyncObjectReader(const FConcertLocalIdentifierTable* InLocalIdentifierTable, FConcertSyncWorldRemapper InWorldRemapper, const FConcertSessionVersionInfo* InVersionInfo, UObject* InObj, const TArray<uint8>& InBytes, const FConcertSyncEncounteredMissingObject& InEncounteredMissingObjectDelegate)
	: FConcertIdentifierReader(InLocalIdentifierTable, InBytes, /*bIsPersistent*/true)
	, WorldRemapper(MoveTemp(InWorldRemapper))
	, EncounteredMissingObjectDelegate(InEncounteredMissingObjectDelegate)
{
	UE::Concert::Private::ConcertSyncArchiveUtil::InitReaderArchive(*this, InVersionInfo);

#if USE_STABLE_LOCALIZATION_KEYS
	if (GIsEditor && !(ArPortFlags & PPF_DuplicateForPIE))
	{
		SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(InObj));
	}
#endif // USE_STABLE_LOCALIZATION_KEYS
}

FConcertSyncObjectReader::FConcertSyncObjectReader(const FConcertLocalIdentifierTable* InLocalIdentifierTable, FConcertSyncWorldRemapper InWorldRemapper, const FConcertSessionVersionInfo* InVersionInfo, UObject* InObj, const TArray<uint8>& InBytes)
	: FConcertSyncObjectReader(InLocalIdentifierTable, InWorldRemapper, InVersionInfo, InObj, InBytes, FConcertSyncEncounteredMissingObject())
{
}

void FConcertSyncObjectReader::SerializeObject(UObject* InObject)
{
	CurrentOuter = InObject;
	if (InObject)
	{
		InObject->Serialize(*this);
	}
}

void FConcertSyncObjectReader::SerializeProperty(const FProperty* InProp, UObject* InObject)
{
	for (int32 Idx = 0; Idx < InProp->ArrayDim; ++Idx)
	{
		InProp->SerializeItem(FStructuredArchiveFromArchive(*this).GetSlot(), InProp->ContainerPtrToValuePtr<void>(InObject, Idx));
	}
}

FArchive& FConcertSyncObjectReader::operator<<(UObject*& Obj)
{
	FSoftObjectPath ObjPath;
	ObjPath.SerializePath(*this);

	bool bDidSerializeObject = false;
	if (bSerializeNestedObjects)
	{
		*this << bDidSerializeObject;
	}

	if (ObjPath.IsNull())
	{
		Obj = nullptr;
	}
	else if (ObjPath != SkipAssetsMarker)
	{
		const FString ResolvedObjPath = WorldRemapper.RemapObjectPathName(ObjPath.ToString());
		OnObjectSerialized(FSoftObjectPath(ObjPath));

		// Always attempt to find an in-memory object first as we may be calling this function while a load is taking place
		Obj = StaticFindObject(UObject::StaticClass(), nullptr, *ResolvedObjPath);

		if (!Obj)
		{
			// We do not attempt to load objects within the current world as they may not have been created yet, 
			// and we don't want to trigger a reload of the world package (when iterative cooking is enabled)
			const bool bAllowLoad = !WorldRemapper.ObjectBelongsToWorld(ResolvedObjPath);
			if (bAllowLoad)
			{
				// If the outer name is a package path that isn't currently loaded, then we need to try loading it to avoid 
				// creating an in-memory version of the package (which would prevent the real package ever loading)
				if (FPackageName::IsValidLongPackageName(ResolvedObjPath))
				{
					Obj = LoadPackage(nullptr, *ResolvedObjPath, LOAD_NoWarn);
				}
				else
				{
					Obj = StaticLoadObject(UObject::StaticClass(), nullptr, *ResolvedObjPath);
				}
			}

			if (!Obj && EncounteredMissingObjectDelegate.IsBound())
			{
				EncounteredMissingObjectDelegate.Execute(FStringView(ResolvedObjPath));
			}
		}

		if (bDidSerializeObject)
		{
			FString ClassName;
			int32  CapturedObjectCreationFlags;
			*this << ClassName;
			*this << CapturedObjectCreationFlags;
			EObjectFlags ObjectCreationFlags = static_cast<EObjectFlags>(CapturedObjectCreationFlags);

			if (!Obj)
			{
				UClass* InClass = FindObject<UClass>(nullptr, *ClassName);
				check(InClass);
				Obj = NewObject<UObject>(CurrentOuter, InClass, NAME_None, ObjectCreationFlags);
			}

			check(Obj);
			SerializeObject(Obj);
		}
	}

	return *this;
}

FArchive& FConcertSyncObjectReader::operator<<(FLazyObjectPtr& LazyObjectPtr)
{
	UObject* Obj = nullptr;
	FUniqueObjectGuid SavedObjectGuid;
	*this << Obj;
	*this << SavedObjectGuid;

	// if the resolved object already has an associated Guid, use that instead of the saved one
	// otherwise used the saved guid since it should refer to the Obj path once its state get applied.
	FUniqueObjectGuid ObjectGuid = Obj != nullptr ? FUniqueObjectGuid(Obj) : FUniqueObjectGuid();
	LazyObjectPtr = ObjectGuid.IsValid() ? ObjectGuid : SavedObjectGuid;
	// technically the saved object guid should be the same as the resolved object guid if any.
	ensure(!ObjectGuid.IsValid() || ObjectGuid == SavedObjectGuid);

	return *this;
}

FArchive& FConcertSyncObjectReader::operator<<(FObjectPtr& Obj)
{
	UObject* RawObj = Obj.Get();
	*this << RawObj;
	Obj = RawObj;
	return *this;
}

FArchive& FConcertSyncObjectReader::operator<<(FSoftObjectPtr& AssetPtr)
{
	FSoftObjectPath Obj;
	*this << Obj;
	AssetPtr = Obj;
	return *this;
}

FArchive& FConcertSyncObjectReader::operator<<(FSoftObjectPath& AssetPtr)
{
	FSoftObjectPath ObjPath;
	ObjPath.SerializePath(*this);

	if (ObjPath != SkipAssetsMarker)
	{
		const FString ResolvedObjPath = WorldRemapper.RemapObjectPathName(ObjPath.ToString());
		AssetPtr.SetPath(ResolvedObjPath);

		OnObjectSerialized(AssetPtr);
	}

	return *this;
}

FArchive& FConcertSyncObjectReader::operator<<(FWeakObjectPtr& Value)
{
	UObject* Obj = nullptr;
	*this << Obj;
	Value = Obj;
	return *this;
}

FString FConcertSyncObjectReader::GetArchiveName() const
{
	return TEXT("FConcertSyncObjectReader");
}


FConcertSyncObjectRewriter::FConcertSyncObjectRewriter(const FConcertLocalIdentifierTable* InLocalIdentifierTable, FConcertLocalIdentifierTable* InRewriteIdentifierTable, const FConcertSessionVersionInfo* InVersionInfo, TArray<uint8>& InBytes)
	: FConcertIdentifierRewriter(InLocalIdentifierTable, InRewriteIdentifierTable, InBytes, /*bIsPersistent*/true)
{
	UE::Concert::Private::ConcertSyncArchiveUtil::InitReaderArchive(*this, InVersionInfo);
}

void FConcertSyncObjectRewriter::RewriteObject(const UClass* InClass)
{
	// Allocate a temporary object to read into while rewriting the data
	// TODO: This should probably use a pool of some kind
	UObject* TmpObject = NewObject<UObject>(GetTransientPackage(), InClass);
	TmpObject->Serialize(*this);
}

void FConcertSyncObjectRewriter::RewriteProperty(const FProperty* InProp)
{
	// Allocate a temporary property to read into while rewriting the data
	void* TmpPropData = FMemory_Alloca_Aligned(InProp->GetSize(), InProp->GetMinAlignment());
	InProp->InitializeValue(TmpPropData);

	for (int32 Idx = 0; Idx < InProp->ArrayDim; ++Idx)
	{
		InProp->SerializeItem(FStructuredArchiveFromArchive(*this).GetSlot(), (uint8*)TmpPropData + (InProp->GetElementSize() * Idx));
	}

	InProp->DestroyValue(TmpPropData);
}

FArchive& FConcertSyncObjectRewriter::operator<<(UObject*& Obj)
{
	FSoftObjectPath ObjPath;
	const int64 OffsetBeforeObjectRead = Offset;
	ObjPath.SerializePath(*this);
	const int64 OffsetAfterObjectRead = Offset;

	if (ObjPath.IsNull())
	{
		Obj = nullptr;
	}
	else if (ObjPath != SkipAssetsMarker)
	{
		OnObjectSerialized(ObjPath);
		RewriteData(OffsetBeforeObjectRead, OffsetAfterObjectRead - OffsetBeforeObjectRead, ObjPath);

		const FSoftObjectPath OutputObjPath = GetOutputObjectPath(ObjPath);
		Obj = StaticFindObject(UObject::StaticClass(), nullptr, *OutputObjPath.ToString());
	}

	return *this;
}

FArchive& FConcertSyncObjectRewriter::operator<<(FLazyObjectPtr& LazyObjectPtr)
{
	UObject* Obj = nullptr;
	FUniqueObjectGuid SavedObjectGuid;
	*this << Obj;
	*this << SavedObjectGuid;

	// if the resolved object already has an associated Guid, use that instead of the saved one
	// otherwise used the saved guid since it should refer to the Obj path once its state get applied.
	FUniqueObjectGuid ObjectGuid = Obj != nullptr ? FUniqueObjectGuid(Obj) : FUniqueObjectGuid();
	LazyObjectPtr = ObjectGuid.IsValid() ? ObjectGuid : SavedObjectGuid;
	// technically the saved object guid should be the same as the resolved object guid if any.
	ensure(!ObjectGuid.IsValid() || ObjectGuid == SavedObjectGuid);

	return *this;
}

FArchive& FConcertSyncObjectRewriter::operator<<(FObjectPtr& Obj)
{
	UObject* RawObj = Obj.Get();
	*this << RawObj;
	Obj = RawObj;
	return *this;
}

FArchive& FConcertSyncObjectRewriter::operator<<(FSoftObjectPtr& AssetPtr)
{
	FSoftObjectPath Obj;
	*this << Obj;
	AssetPtr = Obj;
	return *this;
}

FArchive& FConcertSyncObjectRewriter::operator<<(FSoftObjectPath& AssetPtr)
{
	FSoftObjectPath ObjPath;
	const int64 OffsetBeforeObjectRead = Offset;
	ObjPath.SerializePath(*this);
	const int64 OffsetAfterObjectRead = Offset;

	if (ObjPath != SkipAssetsMarker)
	{
		OnObjectSerialized(ObjPath);
		RewriteData(OffsetBeforeObjectRead, OffsetAfterObjectRead - OffsetBeforeObjectRead, ObjPath);
		AssetPtr = GetOutputObjectPath(ObjPath);
	}

	return *this;
}

FArchive& FConcertSyncObjectRewriter::operator<<(FWeakObjectPtr& Value)
{
	UObject* Obj = nullptr;
	*this << Obj;
	Value = Obj;
	return *this;
}

FString FConcertSyncObjectRewriter::GetArchiveName() const
{
	return TEXT("FConcertSyncObjectRewriter");
}
