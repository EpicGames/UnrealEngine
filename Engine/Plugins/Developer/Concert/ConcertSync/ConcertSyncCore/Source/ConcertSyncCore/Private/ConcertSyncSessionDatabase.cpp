// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncSessionDatabase.h"
#include "ConcertFileCache.h"
#include "ConcertLogGlobal.h"
#include "ConcertSyncSessionTypes.h"
#include "ConcertUtil.h"

#include "Misc/CompressionFlags.h"
#include "SQLiteDatabase.h"
#include "HAL/FileManager.h"
#include "SQLitePreparedStatement.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Tasks/Task.h"

/** Contains an asynchronous representation of the FConcertPackageDataStream */
namespace PackageDataUtil
{
	struct WritePackageResult
	{
		TArray<uint8> CacheData;
		bool		  bShouldCache{false};
	};
}

namespace TransactionDataUtil
{

const int32 MinFilesToCache = 10;
const uint64 MaxFileSizeBytesToCache = 50 * 1024 * 1024;
const int64 BucketSize = 500;
const uint32 DataVersion = 2;
const FGuid DataEntryFooter = FGuid(0xE473C070, 0x65DA42BF, 0xA0607C78, 0xE0DC47CF);

}; // namespace TransactionDataUtil

struct FCompressionMethodAndBias
{
	FCompressionMethodAndBias() = default;
	FCompressionMethodAndBias(EConcertCompressionMethod InMethod, EConcertCompressionBias InBias)
		:  Method(InMethod)
		  ,Bias(InBias)
	{
	}

	
	EConcertCompressionMethod Method = EConcertCompressionMethod::Uncompressed;
	EConcertCompressionBias Bias = EConcertCompressionBias::None;


	/** Non-shifted conversion into a int32 value. This version is used when storing into the database record as all values are int32 */
	uint32 ToInt32() const
	{
		const uint32 CompressionMethodAndBias = static_cast<int32>(Method) | static_cast<int32>(Bias);
		return CompressionMethodAndBias;
	}

	/** Non-shifted conversion into a enum types. This version is used when reading a int32 table entry that corresponds to the compression flags. */
	void FromInt32(int32 InCompressionAndBias)
	{
		Method = static_cast<EConcertCompressionMethod>(InCompressionAndBias & (static_cast<int32>(EConcertCompressionMethod::Zlib) | static_cast<int32>(EConcertCompressionMethod::Oodle)));
		Bias = static_cast<EConcertCompressionBias>(InCompressionAndBias & (static_cast<int32>(EConcertCompressionBias::Speed) | static_cast<int32>(EConcertCompressionBias::Size)));
		check(IsValid());
	}

	/** Non-shifted conversion into a uint32 value. This version is used when serializing into a transaction record. */
	uint32 ToUint32() const
	{
		const uint32 CompressionMethodAndBias = static_cast<uint32>(Method) | static_cast<uint32>(Bias);
		return CompressionMethodAndBias;
	}

	/** Non-shifted conversion into a enum types. This version is used when deserializing from a transaction record. */
	void FromUint32(uint32 InCompressionAndBias)
	{
		Method = static_cast<EConcertCompressionMethod>(InCompressionAndBias & (static_cast<uint32>(EConcertCompressionMethod::Zlib) | static_cast<uint32>(EConcertCompressionMethod::Oodle)));
		Bias = static_cast<EConcertCompressionBias>(InCompressionAndBias & (static_cast<uint32>(EConcertCompressionBias::Speed) | static_cast<uint32>(EConcertCompressionBias::Size)));
		check(IsValid());
	}

	bool IsValid()
	{
		const bool bMethodValid = (Method == EConcertCompressionMethod::Uncompressed
								|| Method == EConcertCompressionMethod::Oodle
								|| Method == EConcertCompressionMethod::Zlib);
		const bool bBiasValid = ( Bias == EConcertCompressionBias::None
								|| Bias == EConcertCompressionBias::Speed
								|| Bias == EConcertCompressionBias::Size);
		return bMethodValid && bBiasValid;
	}

	/** Shifted conversion which is used when writing the package blob. We write this in the 24 MSB of uint32 */
	uint32 ToUint32_Shifted() const
	{
		const uint32 CompressionMethodAndBias = static_cast<uint8>(Method) | static_cast<uint8>(Bias);
		return CompressionMethodAndBias << 8;
	}

	/** Convert a method and bias value that was stored in the 24 MSB back into enum values. */
	void FromUint32_Shifted(uint32 InSerializedDataVersion)
	{
		const uint32 CompressionAndBias = (InSerializedDataVersion >> 8);
	
		Method = static_cast<EConcertCompressionMethod>(CompressionAndBias & (static_cast<uint32>(EConcertCompressionMethod::Zlib) | static_cast<uint32>(EConcertCompressionMethod::Oodle)));
		Bias = static_cast<EConcertCompressionBias>(CompressionAndBias & (static_cast<uint32>(EConcertCompressionBias::Speed) | static_cast<uint32>(EConcertCompressionBias::Size)));
		check(IsValid());
	}
};

struct FConcertPackageAsyncDataStream
{
	FConcertPackageAsyncDataStream(const FConcertPackageDataStream& InStream, const FName& InPackageName)
		:
		CachedPackageName(InPackageName),
		PackageData(InStream.DataBlob->GetData(),InStream.DataBlob->Num())
	{
		PackageStream = {nullptr, InStream.DataSize, &PackageData};
	}

	FName CachedPackageName;
	UE::Tasks::TTask<bool> AsyncTask;
	PackageDataUtil::WritePackageResult Result;

	FConcertPackageDataStream   PackageStream;
	TArray<uint8>			    PackageData;
};

using FConcertPackageAsyncDataStreamPtr = TSharedPtr<FConcertPackageAsyncDataStream, ESPMode::ThreadSafe>;

struct FDeferredLargePackageIOImpl
{
	using FAsyncDataStreamMap = TMap<FString,FConcertPackageAsyncDataStreamPtr>;
	FAsyncDataStreamMap& GetMap()
	{
		return DeferredLargePackageIO;
	}

	const FAsyncDataStreamMap& GetMap() const
	{
		return DeferredLargePackageIO;
	}

	void Clear()
	{
		DeferredLargePackageIO = {};
	}

	FAsyncDataStreamMap DeferredLargePackageIO;
};

FOnPackageSaved& UE::ConcertSyncCore::SyncDatabase::GetOnPackageSavedDelegate()
{
	static FOnPackageSaved SavedDelegate;
	return SavedDelegate;
}

class FMemorySyncDbWriter : public FMemoryWriter
{
public:
	using FMemoryWriter::FMemoryWriter;
	using FMemoryWriter::operator<<;
	virtual FArchive& operator<<(FSoftObjectPath& AssetPtr) override
	{
		AssetPtr.SerializePath(*this);
		return *this;
	}
};

class FMemorySyncDbReader : public FMemoryReader
{
public:
	using FMemoryReader::FMemoryReader;
	using FMemoryReader::operator<<;
	virtual FArchive& operator<<(FSoftObjectPath& AssetPtr) override
	{
		AssetPtr.SerializePath(*this);
		return *this;
	}
};

namespace TransactionDataUtil
{

FString GetDataPath(const FString& InSessionPath)
{
	return InSessionPath / TEXT("Transactions");
}

FString GetDataFilename(const int64 InIndex)
{
	return FString::Printf(TEXT("%s/%s.utrans"), *LexToString(InIndex / BucketSize), *LexToString(InIndex));
}

bool WriteTransactionData(const FStructOnScope& InTransaction, TArray<uint8>& OutSerializedTransactionData)
{
	SCOPED_CONCERT_TRACE(ConcertSyncSessionDatabase_WriteTransactionData);
	FMemorySyncDbWriter Ar(OutSerializedTransactionData);

	const UScriptStruct* TransactionType = CastChecked<const UScriptStruct>(InTransaction.GetStruct());

	FString TransactionTypeStr = TransactionType->GetPathName();
	Ar << TransactionTypeStr;
	const_cast<UScriptStruct*>(TransactionType)->SerializeItem(Ar, (void*)InTransaction.GetStructMemory(), nullptr);

	return !Ar.IsError();
}

bool WriteTransaction(const FStructOnScope& InTransaction, TArray<uint8>& OutSerializedTransactionData)
{
	SCOPED_CONCERT_TRACE(ConcertSyncSessionDatabase_WriteTransaction);
	check(InTransaction.IsValid());

	FMemorySyncDbWriter Ar(OutSerializedTransactionData);

	// Serialize the data version
	uint32 SerializedDataVersion = DataVersion;
	Ar.SerializeIntPacked(SerializedDataVersion);

	// Write the raw transaction data
	TArray<uint8> UncompressedTransaction;
	if (!WriteTransactionData(InTransaction, UncompressedTransaction))
	{
		return false;
	}

	// Serialize the raw transaction
	uint32 UncompressedTransactionSize = UncompressedTransaction.Num();
	Ar.SerializeIntPacked(UncompressedTransactionSize);
	if (UncompressedTransactionSize > 0)
	{
		FCompressionMethodAndBias MethodAndBias(UE::Concert::Compression::GetCompressionMethod(UncompressedTransactionSize),
												UE::Concert::Compression::GetConcertCompressionBias(UE::Concert::Compression::GetCompressionFlags()));  

		uint32 StoredFormat = MethodAndBias.ToUint32();
		Ar.SerializeIntPacked(StoredFormat);
		if (UE::Concert::Compression::DataIsCompressed(MethodAndBias.Method))
		{
			Ar.SerializeCompressed(UncompressedTransaction.GetData(), UncompressedTransactionSize,
								   UE::Concert::Compression::GetCompressionAlgorithm(MethodAndBias.Method),
								   UE::Concert::Compression::GetCoreCompressionFlags(MethodAndBias.Bias));
		}
		else
		{
			Ar.Serialize(UncompressedTransaction.GetData(), UncompressedTransactionSize);
		}
	}

	// Serialize the footer so we know we didn't crash mid-write
	FGuid SerializedFooter = DataEntryFooter;
	Ar << SerializedFooter;

	return !Ar.IsError();
}

bool ReadTransactionData(const TArray<uint8>& InSerializedTransactionData, FStructOnScope& OutTransaction)
{
	SCOPED_CONCERT_TRACE(ConcertSyncSessionDatabase_ReadTransactionData);
	FMemorySyncDbReader Ar(InSerializedTransactionData);

	// Deserialize the transaction
	UScriptStruct* TransactionType = nullptr;
	{
		FString TransactionTypeStr;
		Ar << TransactionTypeStr;
		TransactionType = LoadObject<UScriptStruct>(nullptr, *TransactionTypeStr);
		if (!TransactionType)
		{
			return false;
		}
	}
	if (OutTransaction.IsValid())
	{
		// If were given an existing transaction to fill with data, then the type must match
		if (TransactionType != OutTransaction.GetStruct())
		{
			return false;
		}
	}
	else
	{
		OutTransaction.Initialize(TransactionType);
	}
	TransactionType->SerializeItem(Ar, OutTransaction.GetStructMemory(), nullptr);

	return !Ar.IsError();
}

bool ReadTransaction(const TArray<uint8>& InSerializedTransactionData, FStructOnScope& OutTransaction)
{
	SCOPED_CONCERT_TRACE(ConcertSyncSessionDatabase_ReadTransaction);
	FMemorySyncDbReader Ar(InSerializedTransactionData);

	// Test the footer is in place so we know we didn't crash mid-write
	bool bParsedFooter = false;
	{
		const int64 SerializedTransactionSize = Ar.TotalSize();
		if (SerializedTransactionSize >= sizeof(FGuid))
		{
			FGuid SerializedFooter;
			Ar.Seek(SerializedTransactionSize - sizeof(FGuid));
			Ar << SerializedFooter;
			Ar.Seek(0);
			bParsedFooter = SerializedFooter == DataEntryFooter;
		}
	}
	if (!bParsedFooter)
	{
		return false;
	}

	// Deserialize the data version
	uint32 SerializedDataVersion = 0;
	Ar.SerializeIntPacked(SerializedDataVersion);

	// Deserialize the raw transaction
	uint32 UncompressedTransactionSize = 0;
	Ar.SerializeIntPacked(UncompressedTransactionSize);
	TArray<uint8> UncompressedTransaction;
	UncompressedTransaction.AddZeroed(UncompressedTransactionSize);

	if (UncompressedTransactionSize > 0)
	{
		// Version 1 data is stored always compressed in Zlib format.
		FCompressionMethodAndBias Format(EConcertCompressionMethod::Zlib, EConcertCompressionBias::None);
		if (SerializedDataVersion > 1)
		{
			uint32 SerializedDataFormat = 0;
			Ar.SerializeIntPacked(SerializedDataFormat);
			
			Format.FromUint32(SerializedDataFormat);
		}

		if (UE::Concert::Compression::DataIsCompressed(Format.Method))
		{
			Ar.SerializeCompressed(UncompressedTransaction.GetData(), UncompressedTransactionSize,
								   UE::Concert::Compression::GetCompressionAlgorithm(Format.Method),
								   UE::Concert::Compression::GetCoreCompressionFlags(Format.Bias));
		}
		else
		{
			Ar.Serialize(UncompressedTransaction.GetData(), UncompressedTransactionSize);
		}
	}

	// Read the raw transaction data
	if (!ReadTransactionData(UncompressedTransaction, OutTransaction))
	{
		return false;
	}

	return !Ar.IsError();
}

} // namespace TransactionDataUtil

namespace PackageDataUtil
{

const int32 MinFilesToCache = 10;
const uint8 DataVersion = 2;
const uint64 MaxFileSizeBytesToCache = 200 * 1024 * 1024;
const FGuid EntryFooter = FGuid(0x2EFC8CDD, 0x748E46C0, 0xA5485769, 0x13A3C354);

/** Threshold used to avoid caching very large packages (3GB for example) in memory. */
constexpr const uint64 MaxPackageBlobSizeForCaching = FMath::Max(
	PackageDataUtil::MaxFileSizeBytesToCache / PackageDataUtil::MinFilesToCache, 16ull * 1024 * 1024);

/**
 * Returns true if a package blob should be cached in memory, according to its size.
 * @param PackageBlobSize The blob size (containing package data (possibly compressed) and some meta data).
 * @return true if the pacage should be cached, false otherwise.
 */
bool ShouldCachePackageBlob(uint64 PackageBlobSize)
{
	return PackageBlobSize <= MaxPackageBlobSizeForCaching;
}

/** Write the version/format information in the first 4 bytes of the blob, enabling the reader to parse the blob. */
void WritePackageBlobVersionInfo(FArchive& Ar, FCompressionMethodAndBias InCompressionMethodAndBias)
{
	check(Ar.IsSaving() && Ar.Tell() == 0); // Expects to write the version/format info at the beginning.

	// Keep the major DataVersion in the 8 LSB, encode package data format in the 24 MSB.				
	uint32 SerializedDataVersion = static_cast<uint32>(PackageDataUtil::DataVersion) | InCompressionMethodAndBias.ToUint32_Shifted();
	Ar.SerializeIntPacked(SerializedDataVersion);
}


/** Reads the first 4 bytes of the archive and detect how the package data was written in the blob. */
FCompressionMethodAndBias ParsePackageDataFormat(FArchive& Ar)
{
	check(Ar.IsLoading() && Ar.Tell() == 0); // Expected to read the version/format info at the beginning.

	uint32 SerializedDataVersion = 0;
	Ar.SerializeIntPacked(SerializedDataVersion);
	uint8 ExtractedDataVersion = static_cast<uint8>(SerializedDataVersion & 0xFF); // DataVersion is in the 8 LSB
	if (ExtractedDataVersion == 1)
	{
		// In Version 1, the package was always written compressed into a single block.
		return FCompressionMethodAndBias(EConcertCompressionMethod::Zlib, EConcertCompressionBias::None);
	}

	// In Version 2, the package data format was encoded after the 8 LSB used for 'DataVersion'
	FCompressionMethodAndBias MethodAndBias;
	MethodAndBias.FromUint32_Shifted(SerializedDataVersion & 0xFFFFFF00);
	return MethodAndBias;
}

FString GetDataPath(const FString& InSessionPath)
{
	return InSessionPath / TEXT("Packages");
}

FString GetDataFilename(const FString InPackageName, const int64 InRevision)
{
	return FString::Printf(TEXT("%s_%s.upackage"), *InPackageName.ToLower(), *LexToString(InRevision));
}

FString GetDataFilename(const FName InPackageName, const int64 InRevision)
{
	return GetDataFilename(InPackageName.ToString(), InRevision);
}

/** Internally used by WritePackageBlob() - write the serialized package header. */
void WritePackageHeader(FArchive& Ar, EConcertCompressionMethod CompressionMethod, EConcertCompressionBias CompressionBias)
{
	// First 4-bytes contains information about the format.
	FCompressionMethodAndBias PackageDataFormat(CompressionMethod, CompressionBias);
	WritePackageBlobVersionInfo(Ar, PackageDataFormat);

	// PackageDataOffset is kept to allow 4.25 to read 4.24 packages. The variable-length data stored between version info and package data was never used, so it was stripped in 4.25.
	int64 PackageDataOffsetPlaceholderPos = Ar.Tell(); // Remember at which offset the package data offset variable is going to be written.
	int64 PackageDataOffsetPlaceholder = 0;
	Ar << PackageDataOffsetPlaceholder; // Reserve a space in the header to store the package data offset.
	int64 PackageDataOffset = Ar.Tell(); // Read the end position of the header.
	Ar.Seek(PackageDataOffsetPlaceholderPos); // Rewind to update the place holder value.
	Ar << PackageDataOffset; // Replace the place holder value with the actual value.
}

/** Internally used by ExtractPackageData() - read serialized package header. */
TTuple<FCompressionMethodAndBias, int64> ReadPackageHeader(FArchive& Ar)
{
	// Consume the first bytes to read the package version and package data format.
	FCompressionMethodAndBias PackageDataFormat = ParsePackageDataFormat(Ar);

	// Deserialize the next 8 bytes containing the offset to the package data (for binary compatibility with 4.24)
	int64 PackageDataOffset = 0;
	Ar << PackageDataOffset;
	return MakeTuple(PackageDataFormat, PackageDataOffset);
}

/** Internally used by WritePackageBlob() - write the serialized package footer. */
void WritePackageFooter(FArchive& Ar)
{
	FGuid SerializedFooter = EntryFooter;
	Ar << SerializedFooter;
}

/** Internally used by ExtractPackageData() - ensure the file was fully written. */
bool HasValidPackageFooter(FArchive& Ar)
{
	// Test the footer is in place so we know we didn't crash mid-write
	const int64 SerializedPackageSize = Ar.TotalSize();
	if (SerializedPackageSize >= sizeof(FGuid))
	{
		FGuid SerializedFooter;
		Ar.Seek(SerializedPackageSize - sizeof(FGuid));
		Ar << SerializedFooter;
		Ar.Seek(0);
		return SerializedFooter == EntryFooter;
	}
	return false;
}

int64 EstimatePackageBlobMetaDataSize()
{
	// Roughly: VersionInfo + PackageDataOffset + Footer.
	return sizeof(uint32) + sizeof(int64) + sizeof(FGuid);
}

bool WillUseMemoryWriter(int64 DataSize)
{
	using namespace UE::Concert::Compression;
	return DataSize > 0 && (DataIsCompressed(GetCompressionMethod(DataSize)) ||
							ShouldCachePackageBlob(DataSize + PackageDataUtil::EstimatePackageBlobMetaDataSize()));
}

void WritePackageCompressed(FConcertPackageDataStream& PackageDataStream, FArchive& DstAr, FCompressionMethodAndBias DataFormat)
{
	check(PackageDataStream.DataSize > 0);

	// Write the package data size.
	uint32 UncompressedPackageSize = PackageDataStream.DataSize;
	DstAr.SerializeIntPacked(UncompressedPackageSize);
	check(static_cast<uint64>(UncompressedPackageSize) <= TNumericLimits<TArray<uint8>::SizeType>::Max());

	if (PackageDataStream.DataBlob) // Optimization to avoid transfering from the archive to a temporary buffer.
	{
		DstAr.SerializeCompressed(const_cast<uint8*>(PackageDataStream.DataBlob->GetData()), UncompressedPackageSize,
								  UE::Concert::Compression::GetCompressionAlgorithm(DataFormat.Method),
								  UE::Concert::Compression::GetCoreCompressionFlags(DataFormat.Bias));
	}
	else
	{
		TArray<uint8> UncompressedPackageData;
		UncompressedPackageData.AddUninitialized(UncompressedPackageSize);
		PackageDataStream.DataAr->Serialize(UncompressedPackageData.GetData(), UncompressedPackageSize); // Read.
		DstAr.SerializeCompressed(const_cast<uint8*>(UncompressedPackageData.GetData()), UncompressedPackageSize,
								  UE::Concert::Compression::GetCompressionAlgorithm(DataFormat.Method),
								  UE::Concert::Compression::GetCoreCompressionFlags(DataFormat.Bias));
	}
}

void WritePackageUncompressed(FConcertPackageDataStream& PackageDataStream, FArchive& DstAr)
{
	// Write the package size.
	DstAr << PackageDataStream.DataSize;

	if (PackageDataStream.DataSize > 0)
	{
		// Copy the package data in the destination.
		if (!ConcertUtil::Copy(DstAr, *PackageDataStream.DataAr, PackageDataStream.DataSize))
		{
			UE_LOGF(LogConcert, Error, "Failed copy package data stream to destination.");
		}
	}
}

WritePackageResult WritePackage(FConcertPackageDataStream& PackageDataStream, FArchive* DstAr)
{
	SCOPED_CONCERT_TRACE(ConcertSyncSessionDatabase_WritePackage);

	WritePackageResult OutResult;
	FMemorySyncDbWriter MemWriter(OutResult.CacheData);

	EConcertCompressionMethod CompressionMethod = UE::Concert::Compression::GetCompressionMethod(PackageDataStream.DataSize);
	EConcertCompressionBias CompressionBias = UE::Concert::Compression::GetConcertCompressionBias(UE::Concert::Compression::GetCompressionFlags());

	if (WillUseMemoryWriter(PackageDataStream.DataSize))
	{
		DstAr = &MemWriter;
	}

	check(DstAr && DstAr->IsSaving() && DstAr->Tell() == 0);

	// The package data stream may be null to write an 'empty' package.
	check(PackageDataStream.DataAr == nullptr || PackageDataStream.DataAr->IsLoading());

	 // If the package data archive is null, the size must be zero.
	check(PackageDataStream.DataAr != nullptr || (PackageDataStream.DataAr == nullptr && PackageDataStream.DataSize == 0 && PackageDataStream.DataBlob == nullptr));

	WritePackageHeader(*DstAr, CompressionMethod, CompressionBias);
	if (UE::Concert::Compression::DataIsCompressed(CompressionMethod))
	{
		WritePackageCompressed(PackageDataStream, *DstAr, FCompressionMethodAndBias(CompressionMethod, CompressionBias));
		OutResult.bShouldCache = ShouldCachePackageBlob(OutResult.CacheData.Num());
	}
	else
	{
		WritePackageUncompressed(PackageDataStream, *DstAr);
		OutResult.bShouldCache = ShouldCachePackageBlob(PackageDataStream.DataSize);
	}
	WritePackageFooter(*DstAr);

	return OutResult;
}

/** Extract the package data in OutPackageData if the data is smaller than the streaming threshold, otherwise, stream the data using the streaming function.*/
bool ExtractPackageData(FArchive& PackageBlobAr, const TFunctionRef<void(FConcertPackageDataStream&)>& PackageDataStreamFallbackFn)
{
	// Ensure the footer is in place so we know we didn't crash mid-write
	if (!HasValidPackageFooter(PackageBlobAr))
	{
		return false;
	}

	// Read the header.
	TTuple<FCompressionMethodAndBias, int64> Header = ReadPackageHeader(PackageBlobAr);
	FCompressionMethodAndBias PackageFormat = Header.Get<0>();
	int64 PackageDataOffset = Header.Get<1>();

	// Put the stream in position.
	PackageBlobAr.Seek(PackageDataOffset);

	// The package data is compressed (this hints the package is small enough to be loaded in memory)
	if (UE::Concert::Compression::DataIsCompressed(PackageFormat.Method))
	{
		// Read the uncompressed package data size.
		uint32 UncompressedPackageSize = 0;
		PackageBlobAr.SerializeIntPacked(UncompressedPackageSize);
		check(static_cast<uint64>(UncompressedPackageSize) <= TNumericLimits<TArray<uint8>::SizeType>::Max()); // Prevent overflowing the variable holding the number of element in the array.

		// Decompress the data (in memory).
		TArray<uint8> DecompressedData;
		if (UncompressedPackageSize > 0)
		{
			DecompressedData.AddUninitialized(UncompressedPackageSize);
			PackageBlobAr.SerializeCompressed(DecompressedData.GetData(), UncompressedPackageSize,
											  UE::Concert::Compression::GetCompressionAlgorithm(PackageFormat.Method),
											  UE::Concert::Compression::GetCoreCompressionFlags(PackageFormat.Bias));
		}

		// Let the caller stream the package data from a memory archive.
		FMemorySyncDbReader DataAr(DecompressedData);
		FConcertPackageDataStream PackageDataStream{ &DataAr, UncompressedPackageSize, &DecompressedData };
		PackageDataStreamFallbackFn(PackageDataStream);
	}
	else
	{
		check(UE::Concert::Compression::DataIsUncompressed(PackageFormat.Method));

		// Read the package size.
		int64 PackageDataSize = 0;
		PackageBlobAr << PackageDataSize;

		// Let the caller stream the package data from the archive.
		FConcertPackageDataStream PackageDataStream{ &PackageBlobAr, PackageDataSize, nullptr };
		PackageDataStreamFallbackFn(PackageDataStream);
	}

	return !PackageBlobAr.IsError();
}

bool ExtractPackageData(const TArray<uint8>& InPackageBlob, const TFunctionRef<void(FConcertPackageDataStream&)>& PackageDataStreamFallbackFn)
{
	SCOPED_CONCERT_TRACE(ConcertSyncSessionDatabase_ExtreactPackageData);
	FMemorySyncDbReader PackageBlobAr(InPackageBlob);
	return ExtractPackageData(PackageBlobAr, PackageDataStreamFallbackFn);
}

bool ExtractPackageData(const FString& InPackageBlobPathname, const TFunctionRef<void(FConcertPackageDataStream&)>& PackageDataStreamFallbackFn)
{
	TUniquePtr<FArchive> PackageBlobAr(IFileManager::Get().CreateFileReader(*InPackageBlobPathname)); // The file containing the database package data (along with the package info and some meta data)
	return PackageBlobAr ? ExtractPackageData(*PackageBlobAr, PackageDataStreamFallbackFn) : false;
}

template <typename SerializedDataT>
void SetSerializedPayloadFlags(SerializedDataT& Data, int32 Flags)
{
	FCompressionMethodAndBias Converted;
	Converted.FromInt32(Flags);
	
	Data.PayloadCompressionMethod = Converted.Method;
	Data.PayloadCompressionBias = Converted.Bias;
}

template <typename SerializedDataT>
int32 ConvertSerializedPayloadFlagsToInt32(const SerializedDataT& Data)
{
	FCompressionMethodAndBias MethodAndBias(Data.PayloadCompressionMethod, Data.PayloadCompressionBias);
	return MethodAndBias.ToInt32();
}

} // namespace PackageDataUtil

enum class FConcertSyncSessionDatabaseVersion
{
	Empty = 0,
	Initial = 1,
	V2 = 2,
	/** Added FConcertSyncActivity::Flags to the activities table */
	AddActivityFlags = 3,
	/** Added FConcertSyncReplicationActivity, which added the replication_events table. */
	AddReplicationActivities = 4,

	LatestPlusOne,
	Current = LatestPlusOne - 1,
};

class FConcertSyncSessionDatabaseStatements
{
public:
	explicit FConcertSyncSessionDatabaseStatements(FSQLiteDatabase& InDatabase)
		: Database(InDatabase)
	{
		check(Database.IsValid());
	}

	bool CreatePreparedStatements()
	{
		check(Database.IsValid());

		#define PREPARE_STATEMENT(VAR)																		\
			(VAR) = Database.PrepareStatement<decltype(VAR)>(ESQLitePreparedStatementFlags::Persistent);	\
			if (!(VAR).IsValid()) { return false; }

		PREPARE_STATEMENT(Statement_BeginTransaction);
		PREPARE_STATEMENT(Statement_CommitTransaction);
		PREPARE_STATEMENT(Statement_RollbackTransaction);

		PREPARE_STATEMENT(Statement_AddObjectPathName);
		PREPARE_STATEMENT(Statement_GetObjectPathName);
		PREPARE_STATEMENT(Statement_GetObjectNameId);

		PREPARE_STATEMENT(Statement_AddPackageName);
		PREPARE_STATEMENT(Statement_GetPackageName);
		PREPARE_STATEMENT(Statement_GetPackageNameId);

		PREPARE_STATEMENT(Statement_GetPersistEventId);
		PREPARE_STATEMENT(Statement_AddPersistEvent);

		PREPARE_STATEMENT(Statement_SetEndpointData);
		PREPARE_STATEMENT(Statement_GetEndpointDataForId);
		PREPARE_STATEMENT(Statement_GetAllEndpointData);
		PREPARE_STATEMENT(Statement_GetAllEndpointIds);

		PREPARE_STATEMENT(Statement_AddConnectionEvent);
		PREPARE_STATEMENT(Statement_SetConnectionEvent);
		PREPARE_STATEMENT(Statement_GetConnectionEventForId);

		PREPARE_STATEMENT(Statement_AddLockEvent);
		PREPARE_STATEMENT(Statement_SetLockEvent);
		PREPARE_STATEMENT(Statement_GetLockEventForId);

		PREPARE_STATEMENT(Statement_SetTransactionEvent);
		PREPARE_STATEMENT(Statement_GetTransactionEventForId);
		PREPARE_STATEMENT(Statement_GetTransactionMaxEventId);

		PREPARE_STATEMENT(Statement_SetPackageEvent);
		PREPARE_STATEMENT(Statement_GetPackageEventForId);
		PREPARE_STATEMENT(Statement_GetPackageNameIdAndRevisonForId);
		PREPARE_STATEMENT(Statement_GetUniquePackageNameIdsForPackageEvents);
		PREPARE_STATEMENT(Statement_GetPackageMaxEventId);
		PREPARE_STATEMENT(Statement_GetPackageDataForRevision);
		PREPARE_STATEMENT(Statement_GetPackageHeadEventId);
		PREPARE_STATEMENT(Statement_GetPackageHeadEventIdAndTransactionIdAtSave);
		PREPARE_STATEMENT(Statement_GetMaxPackageEventIdAndTransactionEventIdAtSavePerPackageNameId);
		PREPARE_STATEMENT(Statement_GetPackageHeadRevison);
		PREPARE_STATEMENT(Statement_GetPackageTransactionEventIdAtLastSave);
		
		PREPARE_STATEMENT(Statement_AddReplicationData);
		PREPARE_STATEMENT(Statement_SetReplicationData);
		PREPARE_STATEMENT(Statement_GetReplicationEventForId);
		PREPARE_STATEMENT(Statement_GetReplicationMaxEventIdByClientAndType);

		PREPARE_STATEMENT(Statement_AddActivityData);
		PREPARE_STATEMENT(Statement_SetActivityData);
		PREPARE_STATEMENT(Statement_GetActivityDataForId);
		PREPARE_STATEMENT(Statement_GetActivityDataForEvent);
		PREPARE_STATEMENT(Statement_GetActivityEventTypeForId);
		PREPARE_STATEMENT(Statement_GetAllActivityData);
		PREPARE_STATEMENT(Statement_GetAllActivityDataForEventType);
		PREPARE_STATEMENT(Statement_GetActivityDataInRange);
		PREPARE_STATEMENT(Statement_GetAllActivityIdAndEventTypes);
		PREPARE_STATEMENT(Statement_GetActivityIdAndEventTypesInRange);
		PREPARE_STATEMENT(Statement_GetActivityIdWithEventTypesAndFlagsInRange);
		PREPARE_STATEMENT(Statement_GetActivityMaxId);
		
		PREPARE_STATEMENT(Statement_IgnoreActivity);
		PREPARE_STATEMENT(Statement_PerceiveActivity);
		PREPARE_STATEMENT(Statement_IsActivityIgnored);

		PREPARE_STATEMENT(Statement_MapObjectNameIdToLockEventId);
		PREPARE_STATEMENT(Statement_UnmapObjectNameIdsForLockEventId);
		PREPARE_STATEMENT(Statement_GetLockEventIdsForObjectNameId);
		PREPARE_STATEMENT(Statement_GetObjectNameIdsForLockEventId);

		PREPARE_STATEMENT(Statement_MapPackageNameIdToTransactionEventId);
		PREPARE_STATEMENT(Statement_UnmapPackageNameIdsForTransactionEventId);
		PREPARE_STATEMENT(Statement_GetTransactionEventIdsInRangeForPackageNameId);
		PREPARE_STATEMENT(Statement_GetPackageNameIdsMaxTransactionId);
		PREPARE_STATEMENT(Statement_GetPackageNameIdsWithTransactions);
		PREPARE_STATEMENT(Statement_GetPackageNameIdsForTransactionEventId);

		PREPARE_STATEMENT(Statement_MapObjectNameIdToTransactionEventId);
		PREPARE_STATEMENT(Statement_UnmapObjectNameIdsForTransactionEventId);

		#undef PREPARE_STATEMENT

		return true;
	}

	/**
	 * Statements managing database transactions
	 */

	/** Begin a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FBeginTransaction, "BEGIN TRANSACTION;");
	FBeginTransaction Statement_BeginTransaction;
	bool BeginTransaction()
	{
		return Statement_BeginTransaction.Execute();
	}

	/** Commit a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FCommitTransaction, "COMMIT TRANSACTION;");
	FCommitTransaction Statement_CommitTransaction;
	bool CommitTransaction()
	{
		return Statement_CommitTransaction.Execute();
	}

	/** Rollback a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FRollbackTransaction, "ROLLBACK TRANSACTION;");
	FRollbackTransaction Statement_RollbackTransaction;
	bool RollbackTransaction()
	{
		return Statement_RollbackTransaction.Execute();
	}

	/**
	 * Statements working on object_names
	 */

	/** Add a new object_path_name to object_names and get its object_name_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FAddObjectPathName, "INSERT INTO object_names(object_path_name) VALUES(?1);", SQLITE_PREPARED_STATEMENT_BINDINGS(FName));
	FAddObjectPathName Statement_AddObjectPathName;
	bool AddObjectPathName(const FName InObjectPathName, int64& OutObjectNameId)
	{
		if (Statement_AddObjectPathName.BindAndExecute(InObjectPathName))
		{
			OutObjectNameId = Database.GetLastInsertRowId();
			return true;
		}
		return false;
	}

	/** Get an object_path_name from object_names for the given object_name_id */
	SQLITE_PREPARED_STATEMENT(FGetObjectPathName, "SELECT object_path_name FROM object_names WHERE object_name_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(FName), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetObjectPathName Statement_GetObjectPathName;
	bool GetObjectPathName(const int64 InObjectNameId, FName& OutObjectPathName)
	{
		return Statement_GetObjectPathName.BindAndExecuteSingle(InObjectNameId, OutObjectPathName);
	}

	/** Get an object_name_id from object_names for the given object_path_name */
	SQLITE_PREPARED_STATEMENT(FGetObjectNameId, "SELECT object_name_id FROM object_names WHERE object_path_name = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(FName));
	FGetObjectNameId Statement_GetObjectNameId;
	bool GetObjectNameId(const FName InObjectPathName, int64& OutObjectNameId)
	{
		return Statement_GetObjectNameId.BindAndExecuteSingle(InObjectPathName, OutObjectNameId);
	}

	/**
	 * Statements working on package_names
	 */

	/** Add a new package_name to package_names and get its package_name_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FAddPackageName, "INSERT INTO package_names(package_name) VALUES(?1);", SQLITE_PREPARED_STATEMENT_BINDINGS(FName));
	FAddPackageName Statement_AddPackageName;
	bool AddPackageName(const FName InPackageName, int64& OutPackageNameId)
	{
		if (Statement_AddPackageName.BindAndExecute(InPackageName))
		{
			OutPackageNameId = Database.GetLastInsertRowId();
			return true;
		}
		return false;
	}

	/** Get an package_name from package_names for the given package_name_id */
	SQLITE_PREPARED_STATEMENT(FGetPackageName, "SELECT package_name FROM package_names WHERE package_name_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(FName), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPackageName Statement_GetPackageName;
	bool GetPackageName(const int64 InPackageNameId, FName& OutPackageName)
	{
		return Statement_GetPackageName.BindAndExecuteSingle(InPackageNameId, OutPackageName);
	}

	/** Get a package_name_id from package_names for the given package_name */
	SQLITE_PREPARED_STATEMENT(FGetPackageNameId, "SELECT package_name_id FROM package_names WHERE package_name = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(FName));
	FGetPackageNameId Statement_GetPackageNameId;
	bool GetPackageNameId(const FName InPackageName, int64& OutPackageNameId)
	{
		return Statement_GetPackageNameId.BindAndExecuteSingle(InPackageName, OutPackageNameId);
	}

	/**
	 * Statements working on persist_events
	 */

	/** Get a persist_event_id and transaction_event_id_at_persist from persist_events for the given package_event_id */
	SQLITE_PREPARED_STATEMENT(FGetPersistEventId, "SELECT persist_event_id, transaction_event_id_at_persist FROM persist_events WHERE package_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPersistEventId Statement_GetPersistEventId;
	bool GetPersistEventId(int64 InPackageEventId, int64& OutPersistEventId, int64& OutTransactionEventIdAtPersist)
	{
		return Statement_GetPersistEventId.BindAndExecuteSingle(InPackageEventId, OutPersistEventId, OutTransactionEventIdAtPersist);
	}

	/** Add a new package_event_id to persist_events and get its persist_event_id. */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FAddPersistEvent, "INSERT INTO persist_events(package_event_id, transaction_event_id_at_persist) VALUES(?1, ?2);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FAddPersistEvent Statement_AddPersistEvent;
	bool AddPersistEvent(int64 InPackageEventId, int64 InTransactionEventIdAtPersist, int64& OutPersistEventId)
	{
		if (Statement_AddPersistEvent.BindAndExecute(InPackageEventId, InTransactionEventIdAtPersist))
		{
			OutPersistEventId = Database.GetLastInsertRowId();
			return true;
		}
		return false;
	}

	/**
	 * Statements working on endpoints
	 */

	/** Set the endpoint data in endpoints for the given endpoint_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FSetEndpointData, "INSERT OR REPLACE INTO endpoints(endpoint_id, user_id, client_info_flags, client_info_size_bytes, client_info_data) VALUES(?1, ?2, ?3, ?4, ?5);", SQLITE_PREPARED_STATEMENT_BINDINGS(FGuid, FString, int32, int32, TArray<uint8>));
	FSetEndpointData Statement_SetEndpointData;
	bool SetEndpointData(const FGuid& InEndpointId, const FConcertClientInfo& InClientInfo)
	{
		FConcertSessionSerializedPayload ClientInfoPayload(EConcertPayloadSerializationMethod::Cbor);
		verify(ClientInfoPayload.SetTypedPayload(InClientInfo));
		return Statement_SetEndpointData.BindAndExecute(InEndpointId, InClientInfo.UserName,
														PackageDataUtil::ConvertSerializedPayloadFlagsToInt32(ClientInfoPayload),
														ClientInfoPayload.PayloadSize, ClientInfoPayload.PayloadBytes.Bytes);
	}

	/** Set the endpoint data from endpoints for the given endpoint_id */
	SQLITE_PREPARED_STATEMENT(FGetEndpointDataForId, "SELECT client_info_flags, client_info_size_bytes, client_info_data FROM endpoints WHERE endpoint_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int32, int32, TArray<uint8>), SQLITE_PREPARED_STATEMENT_BINDINGS(FGuid));
	FGetEndpointDataForId Statement_GetEndpointDataForId;
	bool GetEndpointDataForId(const FGuid& InEndpointId, FConcertClientInfo& OutClientInfo)
	{
		FConcertSessionSerializedPayload ClientInfoPayload(EConcertPayloadSerializationMethod::Cbor);
		ClientInfoPayload.PayloadTypeName = *FConcertClientInfo::StaticStruct()->GetPathName();
		int32 ClientInfoFlags = 0;
		if (Statement_GetEndpointDataForId.BindAndExecuteSingle(InEndpointId, ClientInfoFlags, ClientInfoPayload.PayloadSize, ClientInfoPayload.PayloadBytes.Bytes))
		{
			PackageDataUtil::SetSerializedPayloadFlags(ClientInfoPayload, ClientInfoFlags);
			verify(ClientInfoPayload.GetTypedPayload(OutClientInfo));
			return true;
		}
		return false;
	}

	/** Get the endpoint data from endpoints for all endpoint_ids */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetAllEndpointData, "SELECT endpoint_id, client_info_flags, client_info_size_bytes, client_info_data FROM endpoints ORDER BY endpoint_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(FGuid, int32, int32, TArray<uint8>));
	FGetAllEndpointData Statement_GetAllEndpointData;
	bool GetAllEndpointData(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(const FGuid&, FConcertClientInfo&&)> InCallback)
	{
		FConcertSessionSerializedPayload ClientInfoPayload(EConcertPayloadSerializationMethod::Cbor);
		ClientInfoPayload.PayloadTypeName = *FConcertClientInfo::StaticStruct()->GetPathName();
		return Statement_GetAllEndpointData.Execute([&ClientInfoPayload, &InCallback](const FGetAllEndpointData& InStatement)
		{
			FGuid EndpointId;
			int32 ClientInfoFlags = 0;
			if (InStatement.GetColumnValues(EndpointId, ClientInfoFlags, ClientInfoPayload.PayloadSize, ClientInfoPayload.PayloadBytes.Bytes))
			{
				FConcertClientInfo ClientInfo;
				PackageDataUtil::SetSerializedPayloadFlags( ClientInfoPayload, ClientInfoFlags );
				verify(ClientInfoPayload.GetTypedPayload(ClientInfo));
				return InCallback(EndpointId, MoveTemp(ClientInfo));
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the endpoint_ids from all endpoints */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetAllEndpointIds, "SELECT endpoint_id FROM endpoints ORDER BY endpoint_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(FGuid));
	FGetAllEndpointIds Statement_GetAllEndpointIds;
	bool GetAllEndpointIds(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(const FGuid&)> InCallback)
	{
		return Statement_GetAllEndpointIds.Execute([&InCallback](const FGetAllEndpointIds& InStatement)
		{
			FGuid EndpointId;
			if (InStatement.GetColumnValues(EndpointId))
			{
				return InCallback(EndpointId);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/**
	 * Statements working on connection_events
	 */

	/** Add the connection event to connection_events and get its connection_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FAddConnectionEvent, "INSERT INTO connection_events(connection_event_type) VALUES(?1);", SQLITE_PREPARED_STATEMENT_BINDINGS(EConcertSyncConnectionEventType));
	FAddConnectionEvent Statement_AddConnectionEvent;
	bool AddConnectionEvent(const EConcertSyncConnectionEventType InConnectionEventType, int64& OutConnectionEventId)
	{
		if (Statement_AddConnectionEvent.BindAndExecute(InConnectionEventType))
		{
			OutConnectionEventId = Database.GetLastInsertRowId();
			return true;
		}
		return false;
	}

	/** Set the connection event in connection_events for the given connection_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FSetConnectionEvent, "INSERT OR REPLACE INTO connection_events(connection_event_id, connection_event_type) VALUES(?1, ?2);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, EConcertSyncConnectionEventType));
	FSetConnectionEvent Statement_SetConnectionEvent;
	bool SetConnectionEvent(const int64 InConnectionEventId, const EConcertSyncConnectionEventType InConnectionEventType)
	{
		return Statement_SetConnectionEvent.BindAndExecute(InConnectionEventId, InConnectionEventType);
	}

	/** Get the connection event from connection_events for the given connection_event_id */
	SQLITE_PREPARED_STATEMENT(FGetConnectionEventForId, "SELECT connection_event_type FROM connection_events WHERE connection_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(EConcertSyncConnectionEventType), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetConnectionEventForId Statement_GetConnectionEventForId;
	bool GetConnectionEventForId(const int64 InConnectionEventId, EConcertSyncConnectionEventType& OutConnectionEventType)
	{
		return Statement_GetConnectionEventForId.BindAndExecuteSingle(InConnectionEventId, OutConnectionEventType);
	}

	/**
	 * Statements working on lock_events
	 */

	/** Add the lock event to lock_events and get its lock_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FAddLockEvent, "INSERT INTO lock_events(lock_event_type) VALUES(?1);", SQLITE_PREPARED_STATEMENT_BINDINGS(EConcertSyncLockEventType));
	FAddLockEvent Statement_AddLockEvent;
	bool AddLockEvent(const EConcertSyncLockEventType InLockEventType, int64& OutLockEventId)
	{
		if (Statement_AddLockEvent.BindAndExecute(InLockEventType))
		{
			OutLockEventId = Database.GetLastInsertRowId();
			return true;
		}
		return false;
	}

	/** Set the lock event in lock_events for the given lock_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FSetLockEvent, "INSERT OR REPLACE INTO lock_events(lock_event_id, lock_event_type) VALUES(?1, ?2);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, EConcertSyncLockEventType));
	FSetLockEvent Statement_SetLockEvent;
	bool SetLockEvent(const int64 InLockEventId, const EConcertSyncLockEventType InLockEventType)
	{
		return Statement_SetLockEvent.BindAndExecute(InLockEventId, InLockEventType);
	}

	/** Get the lock event from lock_events for the given lock_event_id */
	SQLITE_PREPARED_STATEMENT(FGetLockEventForId, "SELECT lock_event_type FROM lock_events WHERE lock_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(EConcertSyncLockEventType), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetLockEventForId Statement_GetLockEventForId;
	bool GetLockEventForId(const int64 InLockEventId, EConcertSyncLockEventType& OutLockEventType)
	{
		return Statement_GetLockEventForId.BindAndExecuteSingle(InLockEventId, OutLockEventType);
	}

	/**
	 * Statements working on transaction_events
	 */

	/** Set the transaction event in transaction_events for the given transaction_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FSetTransactionEvent, "INSERT OR REPLACE INTO transaction_events(transaction_event_id, data_filename) VALUES(?1, ?2);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, FString));
	FSetTransactionEvent Statement_SetTransactionEvent;
	bool SetTransactionEvent(const int64 InTransactionEventId, const FString& InDataFilename)
	{
		return Statement_SetTransactionEvent.BindAndExecute(InTransactionEventId, InDataFilename);
	}

	/** Get the transaction event from transaction_events for the given transaction_event_id */
	SQLITE_PREPARED_STATEMENT(FGetTransactionEventForId, "SELECT data_filename FROM transaction_events WHERE transaction_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(FString), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetTransactionEventForId Statement_GetTransactionEventForId;
	bool GetTransactionEventForId(const int64 InTransactionEventId, FString& OutDataFilename)
	{
		return Statement_GetTransactionEventForId.BindAndExecuteSingle(InTransactionEventId, OutDataFilename);
	}

	/** Get the largest transaction_event_id currently in transaction_events */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetTransactionMaxEventId, "SELECT MAX(transaction_event_id) FROM transaction_events;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64));
	FGetTransactionMaxEventId Statement_GetTransactionMaxEventId;
	bool GetTransactionMaxEventId(int64& OutTransactionEventId)
	{
		return Statement_GetTransactionMaxEventId.ExecuteSingle(OutTransactionEventId);
	}

	/**
	 * Statements working on package_events
	 */
	
	/** Set the package event in package_events for the given package_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FSetPackageEvent, "INSERT OR REPLACE INTO package_events(package_event_id, package_name_id, package_revision, package_info_flags, package_info_size_bytes, package_info_data, transaction_event_id_at_save, data_filename) VALUES(?1, ?2, ?3, ?4, ?5, ?6, ?7,?8);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64, int64, int32, int32, TArray<uint8>, int64, FString));
	FSetPackageEvent Statement_SetPackageEvent;
	bool SetPackageEvent(const int64 InPackageEventId, const int64 InPackageNameId, const int64 InPackageRevision, const int64 InTransactionEventIdAtSave, const FConcertPackageInfo& InPackageInfo, const FString& InDataFilename)
	{
		FConcertSessionSerializedPayload PackageInfoPayload(EConcertPayloadSerializationMethod::Cbor);
		verify(PackageInfoPayload.SetTypedPayload(InPackageInfo));
		return Statement_SetPackageEvent.BindAndExecute(InPackageEventId, InPackageNameId, InPackageRevision,
														PackageDataUtil::ConvertSerializedPayloadFlagsToInt32(PackageInfoPayload),
														PackageInfoPayload.PayloadSize, PackageInfoPayload.PayloadBytes.Bytes, InTransactionEventIdAtSave, InDataFilename);
	}

	/** Get the package event from package_events for the given package_event_id */
	SQLITE_PREPARED_STATEMENT(FGetPackageEventForId, "SELECT package_revision, package_info_flags, package_info_size_bytes, package_info_data, data_filename FROM package_events WHERE package_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, int32, int32, TArray<uint8>, FString), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPackageEventForId Statement_GetPackageEventForId;
	bool GetPackageEventForId(const int64 InPackageEventId, int64& OutPackageRevision, FConcertPackageInfo& OutPackageInfo, FString& OutDataFilename)
	{
		FConcertSessionSerializedPayload PackageInfoPayload(EConcertPayloadSerializationMethod::Cbor);
		PackageInfoPayload.PayloadTypeName = *FConcertPackageInfo::StaticStruct()->GetPathName();
		int32 ClientFlags;
		if (Statement_GetPackageEventForId.BindAndExecuteSingle(InPackageEventId, OutPackageRevision, ClientFlags, PackageInfoPayload.PayloadSize, PackageInfoPayload.PayloadBytes.Bytes, OutDataFilename))
		{
			PackageDataUtil::SetSerializedPayloadFlags(PackageInfoPayload, ClientFlags);
			verify(PackageInfoPayload.GetTypedPayload(OutPackageInfo));
			return true;
		}
		return false;
	}
	
	/** Get the package_name_id and package_revision from package_events for the given package_event_id */
	SQLITE_PREPARED_STATEMENT(FGetPackageNameIdAndRevisonForId, "SELECT package_name_id, package_revision FROM package_events WHERE package_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPackageNameIdAndRevisonForId Statement_GetPackageNameIdAndRevisonForId;
	bool GetPackageNameIdAndRevisonForId(const int64 InPackageEventId, int64& OutPackageNameId, int64& OutPackageRevision)
	{
		return Statement_GetPackageNameIdAndRevisonForId.BindAndExecuteSingle(InPackageEventId, OutPackageNameId, OutPackageRevision);
	}

	/** Get the package_name_id from package_events for all unique package_name_ids */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetUniquePackageNameIdsForPackageEvents, "SELECT DISTINCT package_name_id FROM package_events;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64));
	FGetUniquePackageNameIdsForPackageEvents Statement_GetUniquePackageNameIdsForPackageEvents;
	bool GetUniquePackageNameIdsForPackageEvents(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64)> InCallback)
	{
		return Statement_GetUniquePackageNameIdsForPackageEvents.Execute([&InCallback](const FGetUniquePackageNameIdsForPackageEvents& InStatement)
		{
			int64 PackageNameId = 0;
			if (InStatement.GetColumnValues(PackageNameId))
			{
				return InCallback(PackageNameId);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the largest package_event_id currently in package_events */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetPackageMaxEventId, "SELECT MAX(package_event_id) FROM package_events;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64));
	FGetPackageMaxEventId Statement_GetPackageMaxEventId;
	bool GetPackageMaxEventId(int64& OutPackageEventId)
	{
		return Statement_GetPackageMaxEventId.ExecuteSingle(OutPackageEventId);
	}

	/** Get the package data from package_events for the given package_name_id and package_revision */
	SQLITE_PREPARED_STATEMENT(FGetPackageDataForRevision, "SELECT package_info_flags, package_info_size_bytes, package_info_data, data_filename FROM package_events WHERE package_name_id = ?1 AND package_revision = ?2;", SQLITE_PREPARED_STATEMENT_COLUMNS(int32, int32, TArray<uint8>, FString), SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FGetPackageDataForRevision Statement_GetPackageDataForRevision;
	bool GetPackageDataForRevision(const int64 InPackageId, const int64 InPackageRevision, FConcertPackageInfo& OutPackageInfo, FString& OutDataFilename)
	{
		FConcertSessionSerializedPayload PackageInfoPayload(EConcertPayloadSerializationMethod::Cbor);
		PackageInfoPayload.PayloadTypeName = *FConcertPackageInfo::StaticStruct()->GetPathName();
		int32 ClientInfoFlags;
		if (Statement_GetPackageDataForRevision.BindAndExecuteSingle(InPackageId, InPackageRevision, ClientInfoFlags, PackageInfoPayload.PayloadSize, PackageInfoPayload.PayloadBytes.Bytes, OutDataFilename))
		{
			PackageDataUtil::SetSerializedPayloadFlags(PackageInfoPayload, ClientInfoFlags);
			verify(PackageInfoPayload.GetTypedPayload(OutPackageInfo));
			return true;
		}
		return false;
	}

	/** Get the largest package_event_id currently in package_events for the given package_name_id */
	SQLITE_PREPARED_STATEMENT(FGetPackageHeadEventId, "SELECT MAX(package_event_id) FROM package_events WHERE package_name_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPackageHeadEventId Statement_GetPackageHeadEventId;
	bool GetPackageHeadEventId(const int64 InPackageNameId, int64& OutPackageEventId)
	{
		return Statement_GetPackageHeadEventId.BindAndExecuteSingle(InPackageNameId, OutPackageEventId);
	}

	/** Get the largest package_event_id and its transaction_event_id_at_save currently in package_events for the given package_name_id */
	SQLITE_PREPARED_STATEMENT(FGetPackageHeadEventIdAndTransactionIdAtSave, "SELECT MAX(package_event_id), transaction_event_id_at_save FROM package_events WHERE package_name_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPackageHeadEventIdAndTransactionIdAtSave Statement_GetPackageHeadEventIdAndTransactionIdAtSave;
	bool GetPackageHeadEventIdAndTransactionIdAtSave(const int64 InPackageNameId, int64& OutPackageEventId, int64& OutTransactionEventIdAtSave)
	{
		return Statement_GetPackageHeadEventIdAndTransactionIdAtSave.BindAndExecuteSingle(InPackageNameId, OutPackageEventId, OutTransactionEventIdAtSave);
	}
	
	/** Get the largest package_event_id along its transaction_event_id_at_save currently in package_events for each distinct package_name_id */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetMaxPackageEventIdAndTransactionEventIdAtSavePerPackageNameId, "SELECT package_name_id, MAX(package_event_id), transaction_event_id_at_save FROM package_events GROUP BY package_name_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, int64, int64));
	FGetMaxPackageEventIdAndTransactionEventIdAtSavePerPackageNameId Statement_GetMaxPackageEventIdAndTransactionEventIdAtSavePerPackageNameId;
	bool GetMaxPackageEventIdAndTransactionEventIdAtSavePerPackageNameId(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64, int64, int64)> InCallback)
	{
		return Statement_GetMaxPackageEventIdAndTransactionEventIdAtSavePerPackageNameId.Execute([&InCallback](const FGetMaxPackageEventIdAndTransactionEventIdAtSavePerPackageNameId& InStatement)
		{
			int64 PackageNameId = 0, MaxPackageEventId = 0, TransactionEventIdAtSave = 0;
			if (InStatement.GetColumnValues(PackageNameId, MaxPackageEventId, TransactionEventIdAtSave))
			{
				return InCallback(PackageNameId, MaxPackageEventId, TransactionEventIdAtSave);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the largest package_revision (for which the activity is not muted) currently in package_events for the given package_name_id */
	SQLITE_PREPARED_STATEMENT(FGetPackageHeadRevison, "SELECT MAX(package_revision) FROM package_events INNER JOIN activities ON package_events.package_event_id = activities.event_id WHERE activities.activity_flags & 1 = 0 AND activities.event_type = 4 AND package_name_id = ?1", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPackageHeadRevison Statement_GetPackageHeadRevison;
	static_assert(static_cast<int32>(EConcertSyncActivityFlags::Muted) == 1, "Please update the SQL statement");
	bool GetUnmutedPackageHeadRevision(const int64 InPackageNameId, int64& OutRevision)
	{
		return Statement_GetPackageHeadRevison.BindAndExecuteSingle(InPackageNameId, OutRevision);
	}

	/** Get the largest transaction_event_id_at_save (for which the transaction is not muted) currently in package_events for the given package_name_id */
	SQLITE_PREPARED_STATEMENT(FGetPackageTransactionEventIdAtLastSave, "SELECT MAX(package_events.transaction_event_id_at_save) FROM package_events INNER JOIN activities ON package_events.package_event_id = activities.event_id WHERE activities.activity_flags & 1 = 0 AND activities.event_type = 4 AND package_name_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPackageTransactionEventIdAtLastSave Statement_GetPackageTransactionEventIdAtLastSave;
	static_assert(static_cast<int32>(EConcertSyncActivityFlags::Muted) == 1, "Update activities.activity_flags & 1 = 0");
	static_assert(static_cast<int32>(EConcertSyncActivityEventType::Package) == 4, "Update activities.event_type = 4");
	bool GetUnmutedPackageTransactionEventIdAtLastSave(const int64 InPackageNameId, int64& OutTransactionEventId)
	{
		return Statement_GetPackageTransactionEventIdAtLastSave.BindAndExecuteSingle(InPackageNameId, OutTransactionEventId);
	}

	/**
	 * Statements working on replication_events
	 */

	/** Set the endpoint data in endpoints for the given endpoint_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FAddReplicationData, "INSERT INTO replication_events(activity_type, event_data_flags, event_data_size_bytes, event_data) VALUES(?1, ?2, ?3, ?4);", SQLITE_PREPARED_STATEMENT_BINDINGS(EConcertSyncReplicationActivityType, int32, int32, TArray<uint8>));
	FAddReplicationData Statement_AddReplicationData;
	bool AddReplicationData(const FConcertSyncReplicationEvent& InEventData, int64& OutActivityEventId)
	{
		const FConcertSessionSerializedPayload& Payload = InEventData.Payload;
		const bool bIsCbor = Payload.SerializationMethod == EConcertPayloadSerializationMethod::Cbor;
		// We don't save SerializationMethod in the database; instead we always assume that it's saved in Cbor format.
		// If we allow the saving here, then GetReplicationEventForId will not deserialize correctly.
		// If you want to change this, add a new field to the database.
		if (!ensure(bIsCbor))
		{
			return false;
		}
		
		const bool bSuccess = Statement_AddReplicationData.BindAndExecute(
			InEventData.ActivityType,
			PackageDataUtil::ConvertSerializedPayloadFlagsToInt32(Payload),
			Payload.PayloadSize,
			Payload.PayloadBytes.Bytes
			);
		if (bSuccess)
		{
			OutActivityEventId = Database.GetLastInsertRowId();
		}
		return bSuccess;
	}

	/** Set the endpoint data in endpoints for the given endpoint_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FSetReplicationData, "INSERT OR REPLACE INTO replication_events(replication_event_id, activity_type, event_data_flags, event_data_size_bytes, event_data) VALUES(?1, ?2, ?3, ?4, ?5);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, EConcertSyncReplicationActivityType, int32, int32, TArray<uint8>));
	FSetReplicationData Statement_SetReplicationData;
	bool SetReplicationData(const int64 InActivityEventId, const FConcertSyncReplicationEvent& InEventData)
	{
		const FConcertSessionSerializedPayload& Payload = InEventData.Payload;
		const bool bIsCbor = Payload.SerializationMethod == EConcertPayloadSerializationMethod::Cbor;
		// We don't save SerializationMethod in the database; instead we always assume that it's saved in Cbor format.
		// If we allow the saving here, then GetReplicationEventForId will not deserialize correctly.
		// If you want to change this, add a new field to the database.
		if (!ensure(bIsCbor))
		{
			return false;
		}
		
		return Statement_SetReplicationData.BindAndExecute(
			InActivityEventId,
			InEventData.ActivityType,
			PackageDataUtil::ConvertSerializedPayloadFlagsToInt32(Payload),
			Payload.PayloadSize,
			Payload.PayloadBytes.Bytes
			);
	}
	
	/** Set the endpoint data from endpoints for the given endpoint_id */
	SQLITE_PREPARED_STATEMENT(FGetReplicationEventForId, "SELECT activity_type, event_data_flags, event_data_size_bytes, event_data FROM replication_events WHERE replication_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(EConcertSyncReplicationActivityType, int32, int32, TArray<uint8>), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetReplicationEventForId Statement_GetReplicationEventForId;
	bool GetReplicationEventForId(const int64 InReplicationEventId, FConcertSyncReplicationEvent& OutReplicationEvent)
	{
		FConcertSessionSerializedPayload& Payload = OutReplicationEvent.Payload;
		int32 SerializationFlags;
		if (Statement_GetReplicationEventForId.BindAndExecuteSingle(InReplicationEventId, OutReplicationEvent.ActivityType, SerializationFlags, Payload.PayloadSize, Payload.PayloadBytes.Bytes))
		{
			PackageDataUtil::SetSerializedPayloadFlags(Payload, SerializationFlags);
			Payload.PayloadTypeName = *UE::ConcertSyncCore::GetReplicationActivityPayloadTypePathName(OutReplicationEvent.ActivityType);
			Payload.SerializationMethod = EConcertPayloadSerializationMethod::Cbor;
			return true;
		}
		return false;
	}
	
	/** Get the largest replication_event_id currently in replication_events that was triggered by EndpointId and has ActivityType */
	SQLITE_PREPARED_STATEMENT(FGetReplicationMaxEventIdByClientAndType, "SELECT MAX(replication_event_id) FROM replication_events JOIN activities ON replication_events.replication_event_id = activities.event_id WHERE endpoint_id = ?1 AND activity_type = ?2;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(FGuid, EConcertSyncReplicationActivityType));
	FGetReplicationMaxEventIdByClientAndType Statement_GetReplicationMaxEventIdByClientAndType;
	bool GetReplicationMaxEventIdByClientAndType(const FGuid& EndpointId, EConcertSyncReplicationActivityType ActivityType, int64& OutReplicationEventId)
	{
		return Statement_GetReplicationMaxEventIdByClientAndType.BindAndExecuteSingle(EndpointId, ActivityType, OutReplicationEventId);
	}

	/**
	 * Statements working on activities
	 */

	/** Add the activity data to activities and get its activity_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FAddActivityData, "INSERT INTO activities(activity_flags, endpoint_id, event_time, event_type, event_id, event_summary_type, event_summary_flags, event_summary_size_bytes, event_summary_data) VALUES(?1, ?2, ?3, ?4, ?5, ?6, ?7, ?8, ?9);", SQLITE_PREPARED_STATEMENT_BINDINGS(EConcertSyncActivityFlags, FGuid, FDateTime, EConcertSyncActivityEventType, int64, FName, int32, int32, TArray<uint8>));
	FAddActivityData Statement_AddActivityData;
	bool AddActivityData(const EConcertSyncActivityFlags Flags, const FGuid& InEndpointId, const EConcertSyncActivityEventType InEventType, const int64 InEventId, const FConcertSessionSerializedPayload& InEventSummary, int64& OutActivityId)
	{
		if (Statement_AddActivityData.BindAndExecute(Flags, InEndpointId, FDateTime::UtcNow(), InEventType,
													 InEventId, InEventSummary.PayloadTypeName,
													 PackageDataUtil::ConvertSerializedPayloadFlagsToInt32(InEventSummary),
													 InEventSummary.PayloadSize,
													 InEventSummary.PayloadBytes.Bytes))
		{
			OutActivityId = Database.GetLastInsertRowId();
			return true;
		}
		return false;
	}

	/** Set the activity data in activities for the given activity_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FSetActivityData, "INSERT OR REPLACE INTO activities(activity_id, activity_flags, endpoint_id, event_time, event_type, event_id, event_summary_type, event_summary_flags, event_summary_size_bytes, event_summary_data) VALUES(?1, ?2, ?3, ?4, ?5, ?6, ?7, ?8, ?9, ?10);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, EConcertSyncActivityFlags, FGuid, FDateTime, EConcertSyncActivityEventType, int64, FName, int32, int32, TArray<uint8>));
	FSetActivityData Statement_SetActivityData;
	bool SetActivityData(const int64 InActivityId, const EConcertSyncActivityFlags Flags, const FGuid& InEndpointId, const FDateTime InEventTime, const EConcertSyncActivityEventType InEventType, const int64 InEventId, const FConcertSessionSerializedPayload& InEventSummary)
	{
		return Statement_SetActivityData.BindAndExecute(InActivityId, Flags, InEndpointId, InEventTime, InEventType, InEventId,
														InEventSummary.PayloadTypeName,
														PackageDataUtil::ConvertSerializedPayloadFlagsToInt32(InEventSummary),
														InEventSummary.PayloadSize,
														InEventSummary.PayloadBytes.Bytes);
	}

	/** Get the activity data from activities for the given activity_id */
	SQLITE_PREPARED_STATEMENT(FGetActivityDataForId, "SELECT activity_flags, endpoint_id, event_time, event_type, event_id, event_summary_type, event_summary_flags, event_summary_size_bytes, event_summary_data FROM activities WHERE activity_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(EConcertSyncActivityFlags, FGuid, FDateTime, EConcertSyncActivityEventType, int64, FName, int32, int32, TArray<uint8>), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetActivityDataForId Statement_GetActivityDataForId;
	bool GetActivityDataForId(const int64 InActivityId, EConcertSyncActivityFlags& OutFlags, FGuid& OutEndpointId, FDateTime& OutEventTime, EConcertSyncActivityEventType& OutEventType, int64& OutEventId, FConcertSessionSerializedPayload& OutEventSummary)
	{
		int32 PayloadFlags;
		if(  Statement_GetActivityDataForId.BindAndExecuteSingle(InActivityId, OutFlags, OutEndpointId, OutEventTime, OutEventType, OutEventId,
																 OutEventSummary.PayloadTypeName,
																 PayloadFlags,
																 OutEventSummary.PayloadSize,
																 OutEventSummary.PayloadBytes.Bytes) )
		{
			PackageDataUtil::SetSerializedPayloadFlags( OutEventSummary, PayloadFlags );
			return true;
		}
		return false;
	}

	/** Get the activity data from activities for the given event_id and event_type */
	SQLITE_PREPARED_STATEMENT(FGetActivityDataForEvent, "SELECT activity_id, activity_flags, endpoint_id, event_time, event_summary_type, event_summary_flags, event_summary_size_bytes, event_summary_data FROM activities WHERE event_id = ?1 AND event_type = ?2;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, EConcertSyncActivityFlags, FGuid, FDateTime, FName, int32, int32, TArray<uint8>), SQLITE_PREPARED_STATEMENT_BINDINGS(int64, EConcertSyncActivityEventType));
	FGetActivityDataForEvent Statement_GetActivityDataForEvent;
	bool GetActivityDataForEvent(const int64 InEventId, const EConcertSyncActivityEventType InEventType, int64& OutActivityId, EConcertSyncActivityFlags& OutFlags, FGuid& OutEndpointId, FDateTime& OutEventTime, FConcertSessionSerializedPayload& OutEventSummary)
	{
		int32 PayloadFlags;
		if (  Statement_GetActivityDataForEvent.BindAndExecuteSingle(InEventId, InEventType, OutActivityId, OutFlags, OutEndpointId,
																	 OutEventTime,
																	 OutEventSummary.PayloadTypeName,
																	 PayloadFlags,
																	 OutEventSummary.PayloadSize,
																	 OutEventSummary.PayloadBytes.Bytes) )
		{
			PackageDataUtil::SetSerializedPayloadFlags( OutEventSummary, PayloadFlags );
			return true;
		}
		return false;
	}

	/** Get the event_type from activities for the given activity_id */
	SQLITE_PREPARED_STATEMENT(FGetActivityEventTypeForId, "SELECT event_type FROM activities WHERE activity_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(EConcertSyncActivityEventType), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetActivityEventTypeForId Statement_GetActivityEventTypeForId;
	bool GetActivityEventTypeForId(const int64 InActivityId, EConcertSyncActivityEventType& OutEventType)
	{
		return Statement_GetActivityEventTypeForId.BindAndExecuteSingle(InActivityId, OutEventType);
	}

	/** Get the activity data from activities for all activity_ids */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetAllActivityData, "SELECT activity_id, activity_flags, endpoint_id, event_time, event_type, event_id, event_summary_type, event_summary_flags, event_summary_size_bytes, event_summary_data FROM activities ORDER BY activity_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, EConcertSyncActivityFlags, FGuid, FDateTime, EConcertSyncActivityEventType, int64, FName, int32, int32, TArray<uint8>));
	FGetAllActivityData Statement_GetAllActivityData;
	bool GetAllActivityData(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64, const EConcertSyncActivityFlags, const FGuid&, FDateTime, EConcertSyncActivityEventType, int64, FConcertSessionSerializedPayload&&)> InCallback)
	{
		return Statement_GetAllActivityData.Execute([&InCallback](const FGetAllActivityData& InStatement)
		{
			int64 ActivityId = 0;
			EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None;
			FGuid EndpointId;
			FDateTime EventTime;
			EConcertSyncActivityEventType EventType = EConcertSyncActivityEventType::Connection;
			int64 EventId = 0;
			FConcertSessionSerializedPayload EventSummary(EConcertPayloadSerializationMethod::Cbor);
			int32 PayloadFlags;
			if (InStatement.GetColumnValues(ActivityId, Flags, EndpointId, EventTime, EventType, EventId,
											EventSummary.PayloadTypeName,
											PayloadFlags,
											EventSummary.PayloadSize,
											EventSummary.PayloadBytes.Bytes))
			{
				PackageDataUtil::SetSerializedPayloadFlags(EventSummary, PayloadFlags);
				return InCallback(ActivityId, Flags, EndpointId, EventTime, EventType, EventId, MoveTemp(EventSummary));
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the activity data from activities for all activities of event_type */
	SQLITE_PREPARED_STATEMENT(FGetAllActivityDataForEventType, "SELECT activity_id, activity_flags, endpoint_id, event_time, event_id, event_summary_type, event_summary_flags, event_summary_size_bytes, event_summary_data FROM activities WHERE event_type = ?1 ORDER BY activity_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, EConcertSyncActivityFlags, FGuid, FDateTime, int64, FName, int32, int32, TArray<uint8>), SQLITE_PREPARED_STATEMENT_BINDINGS(EConcertSyncActivityEventType));
	FGetAllActivityDataForEventType Statement_GetAllActivityDataForEventType;
	bool GetAllActivityDataForEventType(const EConcertSyncActivityEventType InEventType, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64, const EConcertSyncActivityFlags, const FGuid&, FDateTime, int64, FConcertSessionSerializedPayload&&)> InCallback)
	{
		return Statement_GetAllActivityDataForEventType.BindAndExecute(InEventType, [&InCallback](const FGetAllActivityDataForEventType& InStatement)
		{
			int64 ActivityId = 0;
			EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None;
			FGuid EndpointId;
			FDateTime EventTime;
			int64 EventId = 0;
			int32 PayloadFlags;
			FConcertSessionSerializedPayload EventSummary(EConcertPayloadSerializationMethod::Cbor);
			if (InStatement.GetColumnValues(ActivityId, Flags, EndpointId, EventTime, EventId,
											EventSummary.PayloadTypeName,
											PayloadFlags,
											EventSummary.PayloadSize,
											EventSummary.PayloadBytes.Bytes))
			{
				PackageDataUtil::SetSerializedPayloadFlags(EventSummary, PayloadFlags);
				return InCallback(ActivityId, Flags, EndpointId, EventTime, EventId, MoveTemp(EventSummary));
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the activity data from activities for all activities in the given range */
	SQLITE_PREPARED_STATEMENT(FGetActivityDataInRange, "SELECT activity_id, activity_flags, endpoint_id, event_time, event_type, event_id, event_summary_type, event_summary_flags, event_summary_size_bytes, event_summary_data FROM activities WHERE activity_id >= ?1 ORDER BY activity_id LIMIT ?2;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, EConcertSyncActivityFlags, FGuid, FDateTime, EConcertSyncActivityEventType, int64, FName, int32, int32, TArray<uint8>), SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FGetActivityDataInRange Statement_GetActivityDataInRange;
	bool GetActivityDataInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64, const EConcertSyncActivityFlags, const FGuid&, FDateTime, EConcertSyncActivityEventType, int64, FConcertSessionSerializedPayload&&)> InCallback)
	{
		return Statement_GetActivityDataInRange.BindAndExecute(InFirstActivityId, InMaxNumActivities, [&InCallback](const FGetActivityDataInRange& InStatement)
		{
			int64 ActivityId = 0;
			EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None;
			FGuid EndpointId;
			FDateTime EventTime;
			EConcertSyncActivityEventType EventType = EConcertSyncActivityEventType::Connection;
			int64 EventId = 0;
			FConcertSessionSerializedPayload EventSummary(EConcertPayloadSerializationMethod::Cbor);
			int32 EventFlags;
			if (InStatement.GetColumnValues(ActivityId, Flags, EndpointId, EventTime, EventType, EventId,
											EventSummary.PayloadTypeName,
											EventFlags,
											EventSummary.PayloadSize,
											EventSummary.PayloadBytes.Bytes))
			{
				PackageDataUtil::SetSerializedPayloadFlags(EventSummary, EventFlags);
				return InCallback(ActivityId, Flags, EndpointId, EventTime, EventType, EventId, MoveTemp(EventSummary));
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the activity_id and event_type from activities for all activities */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetAllActivityIdAndEventTypes, "SELECT activity_id, event_type FROM activities ORDER BY activity_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, EConcertSyncActivityEventType));
	FGetAllActivityIdAndEventTypes Statement_GetAllActivityIdAndEventTypes;
	bool GetAllActivityIdAndEventTypes(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64, EConcertSyncActivityEventType)> InCallback)
	{
		return Statement_GetAllActivityIdAndEventTypes.Execute([&InCallback](const FGetAllActivityIdAndEventTypes& InStatement)
		{
			int64 ActivityId = 0;
			EConcertSyncActivityEventType EventType = EConcertSyncActivityEventType::Connection;
			if (InStatement.GetColumnValues(ActivityId, EventType))
			{
				return InCallback(ActivityId, EventType);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the activity_id and event_type from activities for all activities in the given range */
	SQLITE_PREPARED_STATEMENT(FGetActivityIdAndEventTypesInRange, "SELECT activity_id, event_type FROM activities WHERE activity_id >= ?1 ORDER BY activity_id LIMIT ?2;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, EConcertSyncActivityEventType), SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FGetActivityIdAndEventTypesInRange Statement_GetActivityIdAndEventTypesInRange;
	bool GetActivityIdAndEventTypesInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64, EConcertSyncActivityEventType)> InCallback)
	{
		return Statement_GetActivityIdAndEventTypesInRange.BindAndExecute(InFirstActivityId, InMaxNumActivities, [&InCallback](const FGetActivityIdAndEventTypesInRange& InStatement)
		{
			int64 ActivityId = 0;
			EConcertSyncActivityEventType EventType = EConcertSyncActivityEventType::Connection;
			if (InStatement.GetColumnValues(ActivityId, EventType))
			{
				return InCallback(ActivityId, EventType);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the activity_id, event_type and activity_flags from activities for all activities in the given range */
	SQLITE_PREPARED_STATEMENT(FGetActivityIdWithEventTypesAndFlagsInRange, "SELECT activity_id, event_type, activity_flags FROM activities WHERE activity_id >= ?1 ORDER BY activity_id LIMIT ?2;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, EConcertSyncActivityEventType, EConcertSyncActivityFlags), SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FGetActivityIdWithEventTypesAndFlagsInRange Statement_GetActivityIdWithEventTypesAndFlagsInRange;
	bool GetActivityIdWithEventTypesAndFlagsInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64, EConcertSyncActivityEventType, EConcertSyncActivityFlags)> InCallback)
	{
		return Statement_GetActivityIdWithEventTypesAndFlagsInRange.BindAndExecute(InFirstActivityId, InMaxNumActivities, [&InCallback](const FGetActivityIdWithEventTypesAndFlagsInRange& InStatement)
		{
			int64 ActivityId = 0;
			EConcertSyncActivityEventType EventType = EConcertSyncActivityEventType::Connection;
			EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None;
			if (InStatement.GetColumnValues(ActivityId, EventType, Flags))
			{
				return InCallback(ActivityId, EventType, Flags);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the largest activity_id currently in activities */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetActivityMaxId, "SELECT MAX(activity_id) FROM activities;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64));
	FGetActivityMaxId Statement_GetActivityMaxId;
	bool GetActivityMaxId(int64& OutActivityId)
	{
		return Statement_GetActivityMaxId.ExecuteSingle(OutActivityId);
	}

	/**
	 * Statements working on ignored_activities
	 */
	
	/** Add the activity_id to ignored_activities */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FIgnoreActivity, "INSERT OR REPLACE INTO ignored_activities(activity_id) VALUES(?1);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FIgnoreActivity Statement_IgnoreActivity;
	bool IgnoreActivity(const int64 InActivityId)
	{
		return Statement_IgnoreActivity.BindAndExecute(InActivityId);
	}

	/** Remove the activity_id from ignored_activities */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FPerceiveActivity, "DELETE FROM ignored_activities WHERE activity_id = ?1;", SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FPerceiveActivity Statement_PerceiveActivity;
	bool PerceiveActivity(const int64 InActivityId)
	{
		return Statement_PerceiveActivity.BindAndExecute(InActivityId);
	}

	/** See if the given activity_id is in ignored_activities */
	SQLITE_PREPARED_STATEMENT(FIsActivityIgnored, "SELECT activity_id FROM ignored_activities WHERE activity_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FIsActivityIgnored Statement_IsActivityIgnored;
	bool IsActivityIgnored(const int64 InActivityId)
	{
		int64 OutActivityId = 0;
		return Statement_IsActivityIgnored.BindAndExecuteSingle(InActivityId, OutActivityId);
	}

	/**
	 * Statements working on resource_locks
	 */

	/** Map the object_name_id in resource_locks to the the given lock_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FMapObjectNameIdToLockEventId, "INSERT INTO resource_locks(object_name_id, lock_event_id) VALUES(?1, ?2);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FMapObjectNameIdToLockEventId Statement_MapObjectNameIdToLockEventId;
	bool MapObjectNameIdToLockEventId(const int64 InObjectNameId, const int64 InLockEventId)
	{
		return Statement_MapObjectNameIdToLockEventId.BindAndExecute(InObjectNameId, InLockEventId);
	}

	/** Unmap all object_name_id entries from resource_locks for the given lock_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FUnmapObjectNameIdsForLockEventId, "DELETE FROM resource_locks WHERE lock_event_id = ?1;", SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FUnmapObjectNameIdsForLockEventId Statement_UnmapObjectNameIdsForLockEventId;
	bool UnmapObjectNameIdsForLockEventId(const int64 InLockEventId)
	{
		return Statement_UnmapObjectNameIdsForLockEventId.BindAndExecute(InLockEventId);
	}

	/** Get the lock_event_id entries from resource_locks for the given object_name_id */
	SQLITE_PREPARED_STATEMENT(FGetLockEventIdsForObjectNameId, "SELECT lock_event_id FROM resource_locks WHERE object_name_id = ?1 ORDER BY lock_event_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetLockEventIdsForObjectNameId Statement_GetLockEventIdsForObjectNameId;
	bool GetLockEventIdsForObjectNameId(const int64 InObjectNameId, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64)> InCallback)
	{
		return Statement_GetLockEventIdsForObjectNameId.BindAndExecute(InObjectNameId, [&InCallback](const FGetLockEventIdsForObjectNameId& InStatement)
		{
			int64 LockEventId = 0;
			if (InStatement.GetColumnValues(LockEventId))
			{
				return InCallback(LockEventId);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the object_name_ids from resource_locks for the given lock_event_id */
	SQLITE_PREPARED_STATEMENT(FGetObjectNameIdsForLockEventId, "SELECT object_name_id FROM resource_locks WHERE lock_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetObjectNameIdsForLockEventId Statement_GetObjectNameIdsForLockEventId;
	bool GetObjectNameIdsForLockEventId(const int64 InLockEventId, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64)> InCallback)
	{
		return Statement_GetObjectNameIdsForLockEventId.BindAndExecute(InLockEventId, [&InCallback](const FGetObjectNameIdsForLockEventId& InStatement)
		{
			int64 ObjectNameId = 0;
			if (InStatement.GetColumnValues(ObjectNameId))
			{
				return InCallback(ObjectNameId);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/**
	 * Statements working on package_transactions
	 */

	/** Map the package_name_id in package_transactions to the the given transaction_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FMapPackageNameIdToTransactionEventId, "INSERT INTO package_transactions(package_name_id, transaction_event_id) VALUES(?1, ?2);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FMapPackageNameIdToTransactionEventId Statement_MapPackageNameIdToTransactionEventId;
	bool MapPackageNameIdToTransactionEventId(const int64 InPackageNameId, const int64 InTransactionEventId)
	{
		return Statement_MapPackageNameIdToTransactionEventId.BindAndExecute(InPackageNameId, InTransactionEventId);
	}

	/** Unmap all package_name_id entries from package_transactions for the given transaction_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FUnmapPackageNameIdsForTransactionEventId, "DELETE FROM package_transactions WHERE transaction_event_id = ?1;", SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FUnmapPackageNameIdsForTransactionEventId Statement_UnmapPackageNameIdsForTransactionEventId;
	bool UnmapPackageNameIdsForTransactionEventId(const int64 InTransactionEventId)
	{
		return Statement_UnmapPackageNameIdsForTransactionEventId.BindAndExecute(InTransactionEventId);
	}

	/** Get the transaction_event_id entries from package_transactions for the given package_name_id and a transaction_event_id >= the given mininum transaction_event_id */
	SQLITE_PREPARED_STATEMENT(FGetTransactionEventIdsInRangeForPackageNameId, "SELECT transaction_event_id FROM package_transactions WHERE package_name_id = ?1 AND transaction_event_id >= ?2 ORDER BY transaction_event_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FGetTransactionEventIdsInRangeForPackageNameId Statement_GetTransactionEventIdsInRangeForPackageNameId;
	static_assert(static_cast<int32>(EConcertSyncActivityFlags::Muted) == 1, "Please update the SQL statement");
	bool GetTransactionEventIdsInRangeForPackageNameId(const int64 InPackageNameId, const int64 InMinTransactionEventId, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64)> InCallback)
	{
		return Statement_GetTransactionEventIdsInRangeForPackageNameId.BindAndExecute(InPackageNameId, InMinTransactionEventId, [&InCallback](const FGetTransactionEventIdsInRangeForPackageNameId& InStatement)
		{
			int64 TransactionEventId = 0;
			if (InStatement.GetColumnValues(TransactionEventId))
			{
				return InCallback(TransactionEventId);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the max transaction_event_id for each package_name_ids from package_transactions */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetPackageNameIdsMaxTransactionId, "SELECT package_name_id, MAX(transaction_event_id) FROM package_transactions GROUP BY package_name_id;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64, int64));
	FGetPackageNameIdsMaxTransactionId Statement_GetPackageNameIdsMaxTransactionId;
	static_assert(static_cast<int32>(EConcertSyncActivityFlags::Muted) == 1, "Please update the SQL statement");
	bool GetPackageNameIdsMaxTransactionId(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64, int64)> InCallback)
	{
		return Statement_GetPackageNameIdsMaxTransactionId.Execute([&InCallback](const FGetPackageNameIdsMaxTransactionId& InStatement)
		{
			int64 PackageNameId = 0, MaxTransactionEventId = 0;
			if (InStatement.GetColumnValues(PackageNameId, MaxTransactionEventId))
			{
				return InCallback(PackageNameId, MaxTransactionEventId);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the unique package_name_ids from package_transactions */
	SQLITE_PREPARED_STATEMENT_COLUMNS_ONLY(FGetPackageNameIdsWithTransactions, "SELECT DISTINCT package_name_id FROM package_transactions;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64));
	FGetPackageNameIdsWithTransactions Statement_GetPackageNameIdsWithTransactions;
	bool GetPackageNameIdsWithTransactions(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64)> InCallback)
	{
		return Statement_GetPackageNameIdsWithTransactions.Execute([&InCallback](const FGetPackageNameIdsWithTransactions& InStatement)
		{
			int64 PackageNameId = 0;
			if (InStatement.GetColumnValues(PackageNameId))
			{
				return InCallback(PackageNameId);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/** Get the package_name_ids from package_transactions for the given transaction_event_id */
	SQLITE_PREPARED_STATEMENT(FGetPackageNameIdsForTransactionEventId, "SELECT package_name_id FROM package_transactions WHERE transaction_event_id = ?1;", SQLITE_PREPARED_STATEMENT_COLUMNS(int64), SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FGetPackageNameIdsForTransactionEventId Statement_GetPackageNameIdsForTransactionEventId;
	bool GetPackageNameIdsForTransactionEventId(const int64 InTransactionEventId, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(int64)> InCallback)
	{
		return Statement_GetPackageNameIdsForTransactionEventId.BindAndExecute(InTransactionEventId, [&InCallback](const FGetPackageNameIdsForTransactionEventId& InStatement)
		{
			int64 PackageNameId = 0;
			if (InStatement.GetColumnValues(PackageNameId))
			{
				return InCallback(PackageNameId);
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	/**
	 * Statements working on object_transactions
	 */

	/** Map the object_name_id in object_transactions to the the given transaction_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FMapObjectNameIdToTransactionEventId, "INSERT INTO object_transactions(object_name_id, transaction_event_id) VALUES(?1, ?2);", SQLITE_PREPARED_STATEMENT_BINDINGS(int64, int64));
	FMapObjectNameIdToTransactionEventId Statement_MapObjectNameIdToTransactionEventId;
	bool MapObjectNameIdToTransactionEventId(const int64 InPackageNameId, const int64 InTransactionEventId)
	{
		return Statement_MapObjectNameIdToTransactionEventId.BindAndExecute(InPackageNameId, InTransactionEventId);
	}

	/** Unmap all object_name_id entries from object_transactions for the given transaction_event_id */
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FUnmapObjectNameIdsForTransactionEventId, "DELETE FROM object_transactions WHERE transaction_event_id = ?1;", SQLITE_PREPARED_STATEMENT_BINDINGS(int64));
	FUnmapObjectNameIdsForTransactionEventId Statement_UnmapObjectNameIdsForTransactionEventId;
	bool UnmapObjectNameIdsForTransactionEventId(const int64 InTransactionEventId)
	{
		return Statement_UnmapObjectNameIdsForTransactionEventId.BindAndExecute(InTransactionEventId);
	}

private:
	FSQLiteDatabase& Database;
};

class FConcertSyncSessionDatabaseScopedTransaction
{
public:
	explicit FConcertSyncSessionDatabaseScopedTransaction(FConcertSyncSessionDatabaseStatements& InStatements)
		: Statements(InStatements)
		, bHasTransaction(Statements.BeginTransaction()) // This will fail if a transaction is already open
	{
	}

	~FConcertSyncSessionDatabaseScopedTransaction()
	{
		Commit();
	}

	bool CommitOrRollback(const bool bShouldCommit)
	{
		if (bShouldCommit)
		{
			Commit();
			return true;
		}
		
		Rollback();
		return false;
	}

	void Commit()
	{
		if (bHasTransaction)
		{
			verify(Statements.CommitTransaction());
			bHasTransaction = false;
		}
	}

	void Rollback()
	{
		if (bHasTransaction)
		{
			verify(Statements.RollbackTransaction());
			bHasTransaction = false;
		}
	}

private:
	FConcertSyncSessionDatabaseStatements& Statements;
	bool bHasTransaction;
};

/** Defined here where TUniquePtr can see the definition of FConcertFileCache and FConcertSyncSessionDatabaseStatements as the TUniquePtr constructor/destructor cannot work with a forward declared type */
FConcertSyncSessionDatabase::FConcertSyncSessionDatabase()
	: Database(MakeUnique<FSQLiteDatabase>()),
	  DeferredLargePackageIOPtr(MakePimpl<FDeferredLargePackageIOImpl>())
{
}

FConcertSyncSessionDatabase::~FConcertSyncSessionDatabase() = default;

FConcertSyncSessionDatabase::FConcertSyncSessionDatabase(FConcertSyncSessionDatabase&&) = default;
FConcertSyncSessionDatabase& FConcertSyncSessionDatabase::operator=(FConcertSyncSessionDatabase&&) = default;

bool FConcertSyncSessionDatabase::IsValid() const
{
	return Database->IsValid();
}

bool FConcertSyncSessionDatabase::Open(const FString& InSessionPath)
{
	return Open(InSessionPath, ESQLiteDatabaseOpenMode::ReadWriteCreate);
}

bool FConcertSyncSessionDatabase::Open(const FString& InSessionPath, const ESQLiteDatabaseOpenMode InOpenMode)
{
	if (Database->IsValid())
	{
		return false;
	}

	if (!Database->Open(*(InSessionPath / TEXT("Session.db")), InOpenMode))
	{
		UE_LOGF(LogConcert, Error, "Failed to open session database for '%ls': %ls", *InSessionPath, *GetLastError());
		return false;
	}

	SessionPath = InSessionPath;
	TransactionFileCache = MakeUnique<FConcertFileCache>(TransactionDataUtil::MinFilesToCache, TransactionDataUtil::MaxFileSizeBytesToCache);
	PackageFileCache = MakeUnique<FConcertFileCache>(PackageDataUtil::MinFilesToCache, PackageDataUtil::MaxFileSizeBytesToCache);

	// Set the database to use exclusive WAL mode for performance (exclusive works even on platforms without a mmap implementation)
	// Set the database "NORMAL" fsync mode to only perform a fsync when checkpointing the WAL to the main database file (fewer fsync calls are better for performance, with a very slight loss of WAL durability if the power fails)
	Database->Execute(TEXT("PRAGMA locking_mode=EXCLUSIVE;"));
	Database->Execute(TEXT("PRAGMA journal_mode=WAL;"));
	Database->Execute(TEXT("PRAGMA synchronous=NORMAL;"));

	int32 LoadedDatabaseVersion = 0;
	Database->GetUserVersion(LoadedDatabaseVersion);
	if (LoadedDatabaseVersion > (int32)FConcertSyncSessionDatabaseVersion::Current)
	{
		Close();
		UE_LOGF(LogConcert, Error, "Failed to open session database for '%ls': Database is too new (version %d, expected <= %d)", *InSessionPath, LoadedDatabaseVersion, (int32)FConcertSyncSessionDatabaseVersion::Current);
		return false;
	}

	const int64 NumberColumns = Database->Execute(TEXT("SELECT name FROM sqlite_master WHERE type='table' AND name='activities';"), [](const auto&){ return ESQLitePreparedStatementExecuteRowResult::Stop; });
	const bool bActivitiesTableExisted = 1 == NumberColumns;

	// Create our required tables
#define CREATE_TABLE(NAME, STATEMENT)																										\
	if (!Database->Execute(TEXT("CREATE TABLE IF NOT EXISTS ") TEXT(NAME) TEXT("(") TEXT(STATEMENT) TEXT(");")))							\
	{																																		\
		Close();																															\
		return false;																														\
	}
	CREATE_TABLE("object_names", "object_name_id INTEGER PRIMARY KEY, object_path_name TEXT UNIQUE NOT NULL");
	CREATE_TABLE("package_names", "package_name_id INTEGER PRIMARY KEY, package_name TEXT UNIQUE NOT NULL");
	CREATE_TABLE("endpoints", "endpoint_id BLOB PRIMARY KEY, user_id TEXT NOT NULL, client_info_flags INTEGER NOT NULL, client_info_size_bytes INTEGER NOT NULL, client_info_data BLOB");
	CREATE_TABLE("connection_events", "connection_event_id INTEGER PRIMARY KEY, connection_event_type INTEGER NOT NULL");
	CREATE_TABLE("lock_events", "lock_event_id INTEGER PRIMARY KEY, lock_event_type INTEGER NOT NULL");
	CREATE_TABLE("transaction_events", "transaction_event_id INTEGER PRIMARY KEY, data_filename TEXT NOT NULL");
	CREATE_TABLE("package_events", "package_event_id INTEGER PRIMARY KEY, package_name_id INTEGER NOT NULL, package_revision INTEGER NOT NULL, package_info_flags INTEGER NOT NULL, package_info_size_bytes INTEGER NOT NULL, package_info_data BLOB, transaction_event_id_at_save INTEGER NOT NULL, data_filename TEXT NOT NULL, FOREIGN KEY(package_name_id) REFERENCES package_names(package_name_id)");
	CREATE_TABLE("persist_events", "persist_event_id INTEGER PRIMARY KEY, package_event_id INTEGER NOT NULL, transaction_event_id_at_persist INTEGER NOT NULL, FOREIGN KEY(package_event_id) REFERENCES package_events(package_event_id)");
	CREATE_TABLE("replication_events", "replication_event_id INTEGER PRIMARY KEY, activity_type INTEGER NOT NULL, event_data_flags INTEGER NOT NULL, event_data_size_bytes INTEGER NOT NULL, event_data BLOB");
	CREATE_TABLE("activities", "activity_id INTEGER PRIMARY KEY, endpoint_id BLOB NOT NULL, event_time INTEGER NOT NULL, event_type INTEGER NOT NULL, event_id INTEGER NOT NULL, event_summary_type TEXT NOT NULL, event_summary_flags INTEGER NOT NULL, event_summary_size_bytes INTEGER NOT NULL, event_summary_data BLOB, activity_flags TINYINT UNSIGNED NOT NULL ,FOREIGN KEY(endpoint_id) REFERENCES endpoints(endpoint_id)");
	CREATE_TABLE("ignored_activities", "activity_id INTEGER NOT NULL, FOREIGN KEY(activity_id) REFERENCES activities(activity_id)");
	CREATE_TABLE("resource_locks", "object_name_id INTEGER NOT NULL, lock_event_id INTEGER NOT NULL, FOREIGN KEY(object_name_id) REFERENCES object_names(object_name_id), FOREIGN KEY(lock_event_id) REFERENCES lock_events(lock_event_id)");
	CREATE_TABLE("package_transactions", "package_name_id INTEGER NOT NULL, transaction_event_id INTEGER NOT NULL, FOREIGN KEY(package_name_id) REFERENCES package_names(package_name_id), FOREIGN KEY(transaction_event_id) REFERENCES transaction_events(transaction_event_id)");
	CREATE_TABLE("object_transactions", "object_name_id INTEGER NOT NULL, transaction_event_id INTEGER NOT NULL, FOREIGN KEY(object_name_id) REFERENCES object_names(object_name_id), FOREIGN KEY(transaction_event_id) REFERENCES transaction_events(transaction_event_id)");
#undef CREATE_TABLE

	if (LoadedDatabaseVersion < (int32)FConcertSyncSessionDatabaseVersion::AddActivityFlags
		&& bActivitiesTableExisted)
	{
		if (!Database->Execute(TEXT("ALTER TABLE activities add activity_flags TINYINT UNSIGNED NOT NULL DEFAULT (0)")))
		{
			Close();
			return false;
		}
	}

	// Create our required indexes
#define CREATE_INDEX(NAME, TABLE, COLS)																										\
	if (!Database->Execute(TEXT("CREATE INDEX IF NOT EXISTS ") TEXT(NAME) TEXT(" ON ") TEXT(TABLE) TEXT("(") TEXT(COLS) TEXT(");")))		\
	{																																		\
		Close();																															\
		return false;																														\
	}
#define CREATE_UNIQUE_INDEX(NAME, TABLE, COLS)																								\
	if (!Database->Execute(TEXT("CREATE UNIQUE INDEX IF NOT EXISTS ") TEXT(NAME) TEXT(" ON ") TEXT(TABLE) TEXT("(") TEXT(COLS) TEXT(");")))	\
	{																																		\
		Close();																															\
		return false;																														\
	}
	CREATE_UNIQUE_INDEX("idx_object_path_names_in_object_names", "object_names", "object_path_name");
	CREATE_UNIQUE_INDEX("idx_package_names_in_package_names", "package_names", "package_name");
	CREATE_INDEX("idx_package_name_ids_in_package_events", "package_events", "package_name_id");
	CREATE_INDEX("idx_package_event_ids_in_persist_events", "persist_events", "package_event_id");
	CREATE_INDEX("idx_event_ids_in_activities", "activities", "event_id");
	CREATE_UNIQUE_INDEX("idx_activity_ids_in_ignored_activities", "ignored_activities", "activity_id");
	CREATE_INDEX("idx_object_name_ids_in_resource_locks", "resource_locks", "object_name_id");
	CREATE_INDEX("idx_lock_event_ids_in_resource_locks", "resource_locks", "lock_event_id");
	CREATE_INDEX("idx_package_name_ids_in_package_transactions", "package_transactions", "package_name_id");
	CREATE_INDEX("idx_transaction_event_ids_in_package_transactions", "package_transactions", "transaction_event_id");
	CREATE_INDEX("idx_object_name_ids_in_object_transactions", "object_transactions", "object_name_id");
	CREATE_INDEX("idx_transaction_event_ids_in_object_transactions", "object_transactions", "transaction_event_id");
#undef CREATE_INDEX
#undef CREATE_UNIQUE_INDEX

	// The database will have the latest schema at this point, so update the user-version
	if (!Database->SetUserVersion((int32)FConcertSyncSessionDatabaseVersion::Current))
	{
		Close();
		return false;
	}

	// Create our required prepared statements
	Statements = MakeUnique<FConcertSyncSessionDatabaseStatements>(*Database);
	if (!Statements->CreatePreparedStatements())
	{
		Close();
		return false;
	}

	return true;
}

bool FConcertSyncSessionDatabase::Close(const bool InDeleteDatabase)
{
	if (!Database->IsValid())
	{
		return false;
	}

	// Need to destroy prepared statements before the database can be closed
	Statements.Reset();

	if (!Database->Close())
	{
		UE_LOGF(LogConcert, Error, "Failed to close session database for '%ls': %ls", *SessionPath, *GetLastError());
		return false;
	}

	FlushAsynchronousTasks();
	TransactionFileCache.Reset();
	PackageFileCache.Reset();

	if (InDeleteDatabase)
	{
		ConcertUtil::DeleteDirectoryTree(*TransactionDataUtil::GetDataPath(SessionPath), *SessionPath);
		ConcertUtil::DeleteDirectoryTree(*PackageDataUtil::GetDataPath(SessionPath), *SessionPath);
		IFileManager::Get().Delete(*(SessionPath / TEXT("Session.db")), false);
	}

	SessionPath.Reset();

	return true;
}

FString FConcertSyncSessionDatabase::GetFilename() const
{
	return Database->GetFilename();
}

FString FConcertSyncSessionDatabase::GetLastError() const
{
	return Database->GetLastError();
}

bool FConcertSyncSessionDatabase::AddConnectionActivity(const FConcertSyncConnectionActivity& InConnectionActivity, int64& OutActivityId, int64& OutConnectionEventId)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		AddConnectionEvent(InConnectionActivity.EventData, OutConnectionEventId) &&
		Statements->AddActivityData(InConnectionActivity.Flags, InConnectionActivity.EndpointId, EConcertSyncActivityEventType::Connection, OutConnectionEventId, InConnectionActivity.EventSummary, OutActivityId) &&
		SetActivityIgnoredState(OutActivityId, InConnectionActivity.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::AddLockActivity(const FConcertSyncLockActivity& InLockActivity, int64& OutActivityId, int64& OutLockEventId)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		AddLockEvent(InLockActivity.EventData, OutLockEventId) &&
		Statements->AddActivityData(InLockActivity.Flags, InLockActivity.EndpointId, EConcertSyncActivityEventType::Lock, OutLockEventId, InLockActivity.EventSummary, OutActivityId) &&
		SetActivityIgnoredState(OutActivityId, InLockActivity.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::AddTransactionActivity(const FConcertSyncTransactionActivity& InTransactionActivity, int64& OutActivityId, int64& OutTransactionEventId)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		AddTransactionEvent(InTransactionActivity.EventData, OutTransactionEventId) &&
		Statements->AddActivityData(InTransactionActivity.Flags, InTransactionActivity.EndpointId, EConcertSyncActivityEventType::Transaction, OutTransactionEventId, InTransactionActivity.EventSummary, OutActivityId) &&
		SetActivityIgnoredState(OutActivityId, InTransactionActivity.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::AddPackageActivity(const FConcertSyncActivity& InPackageActivityBasePart, const FConcertPackageInfo& PackageInfo, FConcertPackageDataStream& InPackageDataStream, int64& OutActivityId, int64& OutPackageEventId)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		AddPackageEvent(PackageInfo, InPackageDataStream, OutPackageEventId) &&
		Statements->AddActivityData(InPackageActivityBasePart.Flags, InPackageActivityBasePart.EndpointId, EConcertSyncActivityEventType::Package, OutPackageEventId, InPackageActivityBasePart.EventSummary, OutActivityId) &&
		SetActivityIgnoredState(OutActivityId, InPackageActivityBasePart.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::AddReplicationActivity(const FConcertSyncReplicationActivity& InReplicationActivity, int64& OutActivityId, int64& OutReplicationEventId)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		AddReplicationEvent(InReplicationActivity.EventData, OutReplicationEventId) &&
		Statements->AddActivityData(InReplicationActivity.Flags, InReplicationActivity.EndpointId, EConcertSyncActivityEventType::Replication, OutReplicationEventId, InReplicationActivity.EventSummary, OutActivityId) &&
		SetActivityIgnoredState(OutActivityId, InReplicationActivity.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::SetActivities(const TSet<int64>& ActivityIds, TFunctionRef<void(FConcertSyncActivity&)> UpdateCallback)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	bool bErrorFree = true;
	ON_SCOPE_EXIT
	{
		if (bErrorFree)
		{
			ScopedTransaction.Commit();
		}
		else
		{
			ScopedTransaction.Rollback();
		}
	};
	
	for (auto ActivityIdIt = ActivityIds.CreateConstIterator(); ActivityIdIt && bErrorFree; ++ActivityIdIt)
	{
		FConcertSyncActivity Activity;
		bErrorFree &= GetActivity(*ActivityIdIt, Activity);
		if (bErrorFree)
		{
			UpdateCallback(Activity);
			bErrorFree &= Statements->SetActivityData(Activity.ActivityId, Activity.Flags, Activity.EndpointId, Activity.EventTime, Activity.EventType, Activity.EventId, Activity.EventSummary);
		}
	}
	return bErrorFree;
}

bool FConcertSyncSessionDatabase::SetConnectionActivity(const FConcertSyncConnectionActivity& InConnectionActivity)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		SetConnectionEvent(InConnectionActivity.EventId, InConnectionActivity.EventData) &&
		Statements->SetActivityData(InConnectionActivity.ActivityId, InConnectionActivity.Flags, InConnectionActivity.EndpointId, InConnectionActivity.EventTime, InConnectionActivity.EventType, InConnectionActivity.EventId, InConnectionActivity.EventSummary) &&
		SetActivityIgnoredState(InConnectionActivity.ActivityId, InConnectionActivity.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::SetLockActivity(const FConcertSyncLockActivity& InLockActivity)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		SetLockEvent(InLockActivity.EventId, InLockActivity.EventData) &&
		Statements->SetActivityData(InLockActivity.ActivityId, InLockActivity.Flags, InLockActivity.EndpointId, InLockActivity.EventTime, InLockActivity.EventType, InLockActivity.EventId, InLockActivity.EventSummary) &&
		SetActivityIgnoredState(InLockActivity.ActivityId, InLockActivity.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::SetTransactionActivity(const FConcertSyncTransactionActivity& InTransactionActivity, const bool bMetaDataOnly)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		SetTransactionEvent(InTransactionActivity.EventId, InTransactionActivity.EventData, bMetaDataOnly) &&
		Statements->SetActivityData(InTransactionActivity.ActivityId, InTransactionActivity.Flags, InTransactionActivity.EndpointId, InTransactionActivity.EventTime, InTransactionActivity.EventType, InTransactionActivity.EventId, InTransactionActivity.EventSummary) &&
		SetActivityIgnoredState(InTransactionActivity.ActivityId, InTransactionActivity.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::SetPackageActivity(const FConcertSyncActivity& InPackageActivityBasePart, FConcertSyncPackageEventData& InPackageActivityEventPart, const bool bMetaDataOnly)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		SetPackageEvent(InPackageActivityBasePart.EventId, InPackageActivityEventPart.MetaData.PackageRevision, InPackageActivityEventPart.MetaData.PackageInfo, bMetaDataOnly ? static_cast<FConcertPackageDataStream*>(nullptr) : &InPackageActivityEventPart.PackageDataStream) &&
		Statements->SetActivityData(InPackageActivityBasePart.ActivityId, InPackageActivityBasePart.Flags, InPackageActivityBasePart.EndpointId, InPackageActivityBasePart.EventTime, InPackageActivityBasePart.EventType, InPackageActivityBasePart.EventId, InPackageActivityBasePart.EventSummary) &&
		SetActivityIgnoredState(InPackageActivityBasePart.ActivityId, InPackageActivityBasePart.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::SetReplicationActivity(const FConcertSyncReplicationActivity& InReplicationActivity)
{
	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(
		SetReplicationEvent(InReplicationActivity.EventId, InReplicationActivity.EventData) &&
		Statements->SetActivityData(InReplicationActivity.ActivityId, InReplicationActivity.Flags, InReplicationActivity.EndpointId, InReplicationActivity.EventTime, InReplicationActivity.EventType, InReplicationActivity.EventId, InReplicationActivity.EventSummary) &&
		SetActivityIgnoredState(InReplicationActivity.ActivityId, InReplicationActivity.bIgnored)
		);
}

bool FConcertSyncSessionDatabase::GetActivity(const int64 InActivityId, FConcertSyncActivity& OutActivity) const
{
	OutActivity.ActivityId = InActivityId;
	OutActivity.bIgnored = Statements->IsActivityIgnored(InActivityId);
	return Statements->GetActivityDataForId(InActivityId, OutActivity.Flags, OutActivity.EndpointId, OutActivity.EventTime, OutActivity.EventType, OutActivity.EventId, OutActivity.EventSummary);
}

bool FConcertSyncSessionDatabase::GetConnectionActivity(const int64 InActivityId, FConcertSyncConnectionActivity& OutConnectionActivity) const
{
	return GetActivity(InActivityId, OutConnectionActivity)
		&& GetConnectionEvent(OutConnectionActivity.EventId, OutConnectionActivity.EventData);
}

bool FConcertSyncSessionDatabase::GetLockActivity(const int64 InActivityId, FConcertSyncLockActivity& OutLockActivity) const
{
	return GetActivity(InActivityId, OutLockActivity)
		&& GetLockEvent(OutLockActivity.EventId, OutLockActivity.EventData);
}

bool FConcertSyncSessionDatabase::GetTransactionActivity(const int64 InActivityId, FConcertSyncTransactionActivity& OutTransactionActivity) const
{
	return GetActivity(InActivityId, OutTransactionActivity)
		&& GetTransactionEvent(OutTransactionActivity.EventId, OutTransactionActivity.EventData);
}

bool FConcertSyncSessionDatabase::GetPackageActivity(const int64 InActivityId, FConsumePackageActivityFunc PackageActivityFn) const
{
	FConcertSyncActivity PackageActivityBasePart;

	auto GetPackageActivityEventFn = [&PackageActivityFn, &PackageActivityBasePart](FConcertSyncPackageEventData& PackageActivityEventPart) mutable
	{
		PackageActivityFn(MoveTemp(PackageActivityBasePart), PackageActivityEventPart);
	};

	return GetActivity(InActivityId, PackageActivityBasePart) // Pull the base part of the package activity.
		&& GetPackageEvent(PackageActivityBasePart.EventId, GetPackageActivityEventFn); // Pull the event part of the activity and call back (to let the caller stream the package data).
}

bool FConcertSyncSessionDatabase::GetReplicationActivity(const int64 InActivityId, FConcertSyncReplicationActivity& OutReplicationActivity) const
{
	return GetActivity(InActivityId, OutReplicationActivity)
		&& GetReplicationEvent(OutReplicationActivity.EventId, OutReplicationActivity.EventData);
}

bool FConcertSyncSessionDatabase::GetReplicationActivityForEvent(const int64 InEventId, FConcertSyncReplicationActivity& OutReplicationActivity) const
{
	return GetActivityForEvent(InEventId, EConcertSyncActivityEventType::Replication, OutReplicationActivity)
		&& GetReplicationEvent(OutReplicationActivity.EventId, OutReplicationActivity.EventData);
}

bool FConcertSyncSessionDatabase::GetActivityEventType(const int64 InActivityId, EConcertSyncActivityEventType& OutEventType) const
{
	return Statements->GetActivityEventTypeForId(InActivityId, OutEventType);
}

bool FConcertSyncSessionDatabase::GetActivityForEvent(const int64 InEventId, const EConcertSyncActivityEventType InEventType, FConcertSyncActivity& OutActivity) const
{
	OutActivity.EventId = InEventId;
	OutActivity.EventType = InEventType;
	if (Statements->GetActivityDataForEvent(InEventId, InEventType, OutActivity.ActivityId, OutActivity.Flags, OutActivity.EndpointId, OutActivity.EventTime, OutActivity.EventSummary))
	{
		OutActivity.bIgnored = Statements->IsActivityIgnored(OutActivity.ActivityId);
		return true;
	}
	return false;
}

bool FConcertSyncSessionDatabase::GetConnectionActivityForEvent(const int64 InConnectionEventId, FConcertSyncConnectionActivity& OutConnectionActivity) const
{
	return GetActivityForEvent(InConnectionEventId, EConcertSyncActivityEventType::Connection, OutConnectionActivity)
		&& GetConnectionEvent(InConnectionEventId, OutConnectionActivity.EventData);
}

bool FConcertSyncSessionDatabase::GetLockActivityForEvent(const int64 InLockEventId, FConcertSyncLockActivity& OutLockActivity) const
{
	return GetActivityForEvent(InLockEventId, EConcertSyncActivityEventType::Lock, OutLockActivity)
		&& GetLockEvent(InLockEventId, OutLockActivity.EventData);
}

bool FConcertSyncSessionDatabase::GetTransactionActivityForEvent(const int64 InTransactionEventId, FConcertSyncTransactionActivity& OutTransactionActivity) const
{
	return GetActivityForEvent(InTransactionEventId, EConcertSyncActivityEventType::Transaction, OutTransactionActivity)
		&& GetTransactionEvent(InTransactionEventId, OutTransactionActivity.EventData);
}

bool FConcertSyncSessionDatabase::GetPackageActivityForEvent(const int64 InPackageEventId, FIteratePackageActivityFunc PackageActivityFn) const
{
	FConcertSyncActivity PackageActivityBasePart;

	auto GetPackageActivityEventFn = [&PackageActivityFn, &PackageActivityBasePart](FConcertSyncPackageEventData& PackageActivityEventPart) mutable
	{
		PackageActivityFn(MoveTemp(PackageActivityBasePart), PackageActivityEventPart);
	};

	return GetActivityForEvent(InPackageEventId, EConcertSyncActivityEventType::Package, PackageActivityBasePart) // Pull the base activity part (common to all activity)
		&& GetPackageEvent(InPackageEventId, GetPackageActivityEventFn); // Pull the package specific part and callback
}

bool FConcertSyncSessionDatabase::EnumerateActivities(FIterateActivityFunc InCallback) const
{
	return Statements->GetAllActivityData([this, &InCallback](const int64 InActivityId, const EConcertSyncActivityFlags InFlags, const FGuid& InEndpointId, const FDateTime InEventTime, const EConcertSyncActivityEventType InEventType, const int64 InEventId, FConcertSessionSerializedPayload&& InEventSummary)
	{
		FConcertSyncActivity Activity;
		Activity.ActivityId = InActivityId;
		Activity.bIgnored = Statements->IsActivityIgnored(InActivityId);
		Activity.Flags = InFlags;
		Activity.EndpointId = InEndpointId;
		Activity.EventTime = InEventTime;
		Activity.EventType = InEventType;
		Activity.EventId = InEventId;
		Activity.EventSummary = MoveTemp(InEventSummary);
		return InCallback(MoveTemp(Activity)) == EBreakBehavior::Continue
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::EnumerateConnectionActivities(TFunctionRef<bool(FConcertSyncConnectionActivity&&)> InCallback) const
{
	return Statements->GetAllActivityDataForEventType(EConcertSyncActivityEventType::Connection, [this, &InCallback](const int64 InActivityId, const EConcertSyncActivityFlags InFlags, const FGuid& InEndpointId, const FDateTime InEventTime, const int64 InEventId, FConcertSessionSerializedPayload&& InEventSummary)
	{
		FConcertSyncConnectionActivity ConnectionActivity;
		ConnectionActivity.ActivityId = InActivityId;
		ConnectionActivity.bIgnored = Statements->IsActivityIgnored(InActivityId);
		ConnectionActivity.Flags = InFlags;
		ConnectionActivity.EndpointId = InEndpointId;
		ConnectionActivity.EventTime = InEventTime;
		ConnectionActivity.EventType = EConcertSyncActivityEventType::Connection;
		ConnectionActivity.EventId = InEventId;
		ConnectionActivity.EventSummary = MoveTemp(InEventSummary);
		if (GetConnectionEvent(ConnectionActivity.EventId, ConnectionActivity.EventData))
		{
			return InCallback(MoveTemp(ConnectionActivity))
				? ESQLitePreparedStatementExecuteRowResult::Continue
				: ESQLitePreparedStatementExecuteRowResult::Stop;
		}
		return ESQLitePreparedStatementExecuteRowResult::Error;
	});
}

bool FConcertSyncSessionDatabase::EnumerateLockActivities(TFunctionRef<bool(FConcertSyncLockActivity&&)> InCallback) const
{
	return Statements->GetAllActivityDataForEventType(EConcertSyncActivityEventType::Lock, [this, &InCallback](const int64 InActivityId, const EConcertSyncActivityFlags InFlags, const FGuid& InEndpointId, const FDateTime InEventTime, const int64 InEventId, FConcertSessionSerializedPayload&& InEventSummary)
	{
		FConcertSyncLockActivity LockActivity;
		LockActivity.ActivityId = InActivityId;
		LockActivity.bIgnored = Statements->IsActivityIgnored(InActivityId);
		LockActivity.Flags = InFlags;
		LockActivity.EndpointId = InEndpointId;
		LockActivity.EventTime = InEventTime;
		LockActivity.EventType = EConcertSyncActivityEventType::Lock;
		LockActivity.EventId = InEventId;
		LockActivity.EventSummary = MoveTemp(InEventSummary);
		if (GetLockEvent(LockActivity.EventId, LockActivity.EventData))
		{
			return InCallback(MoveTemp(LockActivity))
				? ESQLitePreparedStatementExecuteRowResult::Continue
				: ESQLitePreparedStatementExecuteRowResult::Stop;
		}
		return ESQLitePreparedStatementExecuteRowResult::Error;
	});
}

bool FConcertSyncSessionDatabase::EnumerateTransactionActivities(TFunctionRef<bool(FConcertSyncTransactionActivity&&)> InCallback) const
{
	return Statements->GetAllActivityDataForEventType(EConcertSyncActivityEventType::Transaction, [this, &InCallback](const int64 InActivityId, const EConcertSyncActivityFlags InFlags, const FGuid& InEndpointId, const FDateTime InEventTime, const int64 InEventId, FConcertSessionSerializedPayload&& InEventSummary)
	{
		FConcertSyncTransactionActivity TransactionActivity;
		TransactionActivity.ActivityId = InActivityId;
		TransactionActivity.bIgnored = Statements->IsActivityIgnored(InActivityId);
		TransactionActivity.Flags = InFlags;
		TransactionActivity.EndpointId = InEndpointId;
		TransactionActivity.EventTime = InEventTime;
		TransactionActivity.EventType = EConcertSyncActivityEventType::Transaction;
		TransactionActivity.EventId = InEventId;
		TransactionActivity.EventSummary = MoveTemp(InEventSummary);
		if (GetTransactionEvent(TransactionActivity.EventId, TransactionActivity.EventData))
		{
			return InCallback(MoveTemp(TransactionActivity))
				? ESQLitePreparedStatementExecuteRowResult::Continue
				: ESQLitePreparedStatementExecuteRowResult::Stop;
		}
		return ESQLitePreparedStatementExecuteRowResult::Error;
	});
}

bool FConcertSyncSessionDatabase::EnumeratePackageActivities(FIteratePackageActivityFunc InCallback) const
{
	return Statements->GetAllActivityDataForEventType(EConcertSyncActivityEventType::Package, [this, &InCallback](const int64 InActivityId, const EConcertSyncActivityFlags InFlags, const FGuid& InEndpointId, const FDateTime InEventTime, const int64 InEventId, FConcertSessionSerializedPayload&& InEventSummary)
	{
		FConcertSyncActivity PackageActivity;
		PackageActivity.ActivityId = InActivityId;
		PackageActivity.bIgnored = Statements->IsActivityIgnored(InActivityId);
		PackageActivity.Flags = InFlags;
		PackageActivity.EndpointId = InEndpointId;
		PackageActivity.EventTime = InEventTime;
		PackageActivity.EventType = EConcertSyncActivityEventType::Package;
		PackageActivity.EventId = InEventId;
		PackageActivity.EventSummary = MoveTemp(InEventSummary);

		ESQLitePreparedStatementExecuteRowResult Result = ESQLitePreparedStatementExecuteRowResult::Error;
		GetPackageEvent(PackageActivity.EventId, [&InCallback, &Result, BasePart = MoveTemp(PackageActivity)](FConcertSyncPackageEventData& PackageEventPart) mutable
		{
			Result = InCallback(MoveTemp(BasePart), PackageEventPart) == EBreakBehavior::Continue
				? ESQLitePreparedStatementExecuteRowResult::Continue
				: ESQLitePreparedStatementExecuteRowResult::Stop;
		});
		return Result;
	});
}

bool FConcertSyncSessionDatabase::EnumerateReplicationActivities(TFunctionRef<bool(FConcertSyncReplicationActivity&&)> InCallback) const
{
	return Statements->GetAllActivityDataForEventType(EConcertSyncActivityEventType::Replication, [this, &InCallback](const int64 InActivityId, const EConcertSyncActivityFlags InFlags, const FGuid& InEndpointId, const FDateTime InEventTime, const int64 InEventId, FConcertSessionSerializedPayload&& InEventSummary)
	{
		FConcertSyncReplicationActivity ReplicationActivity;
		ReplicationActivity.ActivityId = InActivityId;
		ReplicationActivity.bIgnored = Statements->IsActivityIgnored(InActivityId);
		ReplicationActivity.Flags = InFlags;
		ReplicationActivity.EndpointId = InEndpointId;
		ReplicationActivity.EventTime = InEventTime;
		ReplicationActivity.EventType = EConcertSyncActivityEventType::Transaction;
		ReplicationActivity.EventId = InEventId;
		ReplicationActivity.EventSummary = MoveTemp(InEventSummary);
		if (GetReplicationEvent(ReplicationActivity.EventId, ReplicationActivity.EventData))
		{
			return InCallback(MoveTemp(ReplicationActivity))
				? ESQLitePreparedStatementExecuteRowResult::Continue
				: ESQLitePreparedStatementExecuteRowResult::Stop;
		}
		return ESQLitePreparedStatementExecuteRowResult::Error;
	});
}

bool FConcertSyncSessionDatabase::EnumerateActivitiesForEventType(const EConcertSyncActivityEventType InEventType, FIterateActivityFunc InCallback) const
{
	return Statements->GetAllActivityDataForEventType(InEventType, [this, InEventType, &InCallback](const int64 InActivityId, const EConcertSyncActivityFlags InFlags, const FGuid& InEndpointId, const FDateTime InEventTime, const int64 InEventId, FConcertSessionSerializedPayload&& InEventSummary)
	{
		FConcertSyncActivity Activity;
		Activity.ActivityId = InActivityId;
		Activity.bIgnored = Statements->IsActivityIgnored(InActivityId);
		Activity.Flags = InFlags;
		Activity.EndpointId = InEndpointId;
		Activity.EventTime = InEventTime;
		Activity.EventType = InEventType;
		Activity.EventId = InEventId;
		Activity.EventSummary = MoveTemp(InEventSummary);
		return InCallback(MoveTemp(Activity)) == EBreakBehavior::Continue
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::EnumerateActivitiesInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, FIterateActivityFunc InCallback) const
{
	return Statements->GetActivityDataInRange(InFirstActivityId, InMaxNumActivities, [this, &InCallback](const int64 InActivityId, const EConcertSyncActivityFlags InFlags, const FGuid& InEndpointId, const FDateTime InEventTime, const EConcertSyncActivityEventType InEventType, const int64 InEventId, FConcertSessionSerializedPayload&& InEventSummary)
	{
		FConcertSyncActivity Activity;
		Activity.ActivityId = InActivityId;
		Activity.bIgnored = Statements->IsActivityIgnored(InActivityId);
		Activity.Flags = InFlags;
		Activity.EndpointId = InEndpointId;
		Activity.EventTime = InEventTime;
		Activity.EventType = InEventType;
		Activity.EventId = InEventId;
		Activity.EventSummary = MoveTemp(InEventSummary);
		return InCallback(MoveTemp(Activity)) == EBreakBehavior::Continue
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::EnumerateActivityIdsAndEventTypes(TFunctionRef<bool(int64, EConcertSyncActivityEventType)> InCallback) const
{
	return Statements->GetAllActivityIdAndEventTypes([&InCallback](const int64 InActivityId, const EConcertSyncActivityEventType InEventType)
	{
		return InCallback(InActivityId, InEventType)
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::EnumerateActivityIdsAndEventTypesInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, TFunctionRef<bool(int64, EConcertSyncActivityEventType)> InCallback) const
{
	return Statements->GetActivityIdAndEventTypesInRange(InFirstActivityId, InMaxNumActivities, [&InCallback](const int64 InActivityId, const EConcertSyncActivityEventType InEventType)
	{
		return InCallback(InActivityId, InEventType)
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::EnumerateActivityIdsWithEventTypesAndFlagsInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, TFunctionRef<bool(int64, EConcertSyncActivityEventType, EConcertSyncActivityFlags)> InCallback) const
{
	return Statements->GetActivityIdWithEventTypesAndFlagsInRange(InFirstActivityId, InMaxNumActivities, [&InCallback](const int64 InActivityId, const EConcertSyncActivityEventType InEventType, EConcertSyncActivityFlags InFlags)
	{
		return InCallback(InActivityId, InEventType, InFlags)
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::GetActivityMaxId(int64& OutActivityId) const
{
	return Statements->GetActivityMaxId(OutActivityId);
}

bool FConcertSyncSessionDatabase::SetEndpoint(const FGuid& InEndpointId, const FConcertSyncEndpointData& InEndpointData)
{
	return Statements->SetEndpointData(InEndpointId, InEndpointData.ClientInfo);
}

bool FConcertSyncSessionDatabase::GetEndpoint(const FGuid& InEndpointId, FConcertSyncEndpointData& OutEndpointData) const
{
	return Statements->GetEndpointDataForId(InEndpointId, OutEndpointData.ClientInfo);
}

bool FConcertSyncSessionDatabase::EnumerateEndpoints(TFunctionRef<bool(FConcertSyncEndpointIdAndData&&)> InCallback) const
{
	return Statements->GetAllEndpointData([&InCallback](const FGuid& InEndpointId, FConcertClientInfo&& InClientInfo)
	{
		FConcertSyncEndpointIdAndData EndpointData;
		EndpointData.EndpointId = InEndpointId;
		EndpointData.EndpointData.ClientInfo = MoveTemp(InClientInfo);
		return InCallback(MoveTemp(EndpointData))
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::EnumerateEndpointIds(TFunctionRef<bool(FGuid)> InCallback) const
{
	return Statements->GetAllEndpointIds([&InCallback](const FGuid& InEndpointId)
	{
		return InCallback(InEndpointId)
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::SetActivityIgnoredState(const int64 InActivityId, const bool InIsIgnored)
{
	return InIsIgnored
		? Statements->IgnoreActivity(InActivityId)
		: Statements->PerceiveActivity(InActivityId);
}

bool FConcertSyncSessionDatabase::AddConnectionEvent(const FConcertSyncConnectionEvent& InConnectionEvent, int64& OutConnectionEventId)
{
	return Statements->AddConnectionEvent(InConnectionEvent.ConnectionEventType, OutConnectionEventId);
}

bool FConcertSyncSessionDatabase::SetConnectionEvent(const int64 InConnectionEventId, const FConcertSyncConnectionEvent& InConnectionEvent)
{
	return Statements->SetConnectionEvent(InConnectionEventId, InConnectionEvent.ConnectionEventType);
}

bool FConcertSyncSessionDatabase::GetConnectionEvent(const int64 InConnectionEventId, FConcertSyncConnectionEvent& OutConnectionEvent) const
{
	return Statements->GetConnectionEventForId(InConnectionEventId, OutConnectionEvent.ConnectionEventType);
}

bool FConcertSyncSessionDatabase::AddLockEvent(const FConcertSyncLockEvent& InLockEvent, int64& OutLockEventId)
{
	if (Statements->AddLockEvent(InLockEvent.LockEventType, OutLockEventId))
	{
		return MapResourceNamesForLock(OutLockEventId, InLockEvent.ResourceNames);
	}
	return false;
}

bool FConcertSyncSessionDatabase::SetLockEvent(const int64 InLockEventId, const FConcertSyncLockEvent& InLockEvent)
{
	if (Statements->SetLockEvent(InLockEventId, InLockEvent.LockEventType))
	{
		return MapResourceNamesForLock(InLockEventId, InLockEvent.ResourceNames);
	}
	return false;
}

bool FConcertSyncSessionDatabase::GetLockEvent(const int64 InLockEventId, FConcertSyncLockEvent& OutLockEvent) const
{
	if (Statements->GetLockEventForId(InLockEventId, OutLockEvent.LockEventType))
	{
		OutLockEvent.ResourceNames.Reset();
		return Statements->GetObjectNameIdsForLockEventId(InLockEventId, [this, &OutLockEvent](const int64 InObjectNameId)
		{
			return GetObjectPathName(InObjectNameId, OutLockEvent.ResourceNames.AddDefaulted_GetRef())
				? ESQLitePreparedStatementExecuteRowResult::Continue
				: ESQLitePreparedStatementExecuteRowResult::Error;
		});
	}
	return false;
}

bool FConcertSyncSessionDatabase::AddTransactionEvent(const FConcertSyncTransactionEvent& InTransactionEvent, int64& OutTransactionEventId)
{
	// Get the next transaction ID
	if (!GetTransactionMaxEventId(OutTransactionEventId) || OutTransactionEventId == MAX_int64)
	{
		return false;
	}
	++OutTransactionEventId;

	return SetTransactionEvent(OutTransactionEventId, InTransactionEvent);
}

bool FConcertSyncSessionDatabase::UpdateTransactionEvent(const int64 InTransactionEventId, const FConcertSyncTransactionEvent& InTransactionEvent)
{
	int64 MaxTransactionEventId;
	if (GetTransactionMaxEventId(MaxTransactionEventId) && InTransactionEventId <= MaxTransactionEventId) // Ensure the transaction ID is in bound.
	{
		FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
		return ScopedTransaction.CommitOrRollback(
			SetTransactionEvent(InTransactionEventId, InTransactionEvent)
		);
	}

	return false;
}

bool FConcertSyncSessionDatabase::SetTransactionEvent(const int64 InTransactionEventId, const FConcertSyncTransactionEvent& InTransactionEvent, const bool bMetaDataOnly)
{
	// Write the data blob file
	const FString TransactionDataFilename = TransactionDataUtil::GetDataFilename(InTransactionEventId);
	const FString TransactionDataPathname = TransactionDataUtil::GetDataPath(SessionPath) / TransactionDataFilename;

	FStructOnScope Transaction(FConcertTransactionFinalizedEvent::StaticStruct(), (uint8*)&InTransactionEvent.Transaction);
	if (!bMetaDataOnly)
	{
		if (!SaveTransaction(TransactionDataPathname, Transaction))
		{
			return false;
		}
	}

	// Add the database entry
	if (Statements->SetTransactionEvent(InTransactionEventId, TransactionDataFilename))
	{
		return MapPackageNamesForTransaction(InTransactionEventId, InTransactionEvent.Transaction)
			&& MapObjectNamesForTransaction(InTransactionEventId, InTransactionEvent.Transaction);
	}

	return false;
}

bool FConcertSyncSessionDatabase::GetTransactionEvent(const int64 InTransactionEventId, FConcertSyncTransactionEvent& OutTransactionEvent, const bool InMetaDataOnly) const
{
	FString DataFilename;
	if (Statements->GetTransactionEventForId(InTransactionEventId, DataFilename))
	{
		if (InMetaDataOnly)
		{
			OutTransactionEvent.Transaction = FConcertTransactionFinalizedEvent();
			return true;
		}

		const FString TransactionDataPathname = TransactionDataUtil::GetDataPath(SessionPath) / DataFilename;

		FStructOnScope Transaction(FConcertTransactionFinalizedEvent::StaticStruct(), (uint8*)&OutTransactionEvent.Transaction);
		if (LoadTransaction(TransactionDataPathname, Transaction))
		{
			return true;
		}
	}

	return false;
}

bool FConcertSyncSessionDatabase::GetReplicationEvent(const int64 InReplicationEventId, FConcertSyncReplicationEvent& OutReplicationEvent) const
{
	return Statements->GetReplicationEventForId(InReplicationEventId, OutReplicationEvent);
}

bool FConcertSyncSessionDatabase::GetTransactionMaxEventId(int64& OutTransactionEventId) const
{
	return Statements->GetTransactionMaxEventId(OutTransactionEventId);
}

bool FConcertSyncSessionDatabase::IsLiveTransactionEvent(const int64 InTransactionEventId, bool& OutIsLive) const
{
	OutIsLive = false;
	return Statements->GetPackageNameIdsForTransactionEventId(InTransactionEventId, [this, InTransactionEventId, &OutIsLive](const int64 InPackageNameId)
	{
		int64 HeadTransactionEventIdAtLastSave = 0;
		if (!Statements->GetUnmutedPackageTransactionEventIdAtLastSave(InPackageNameId, HeadTransactionEventIdAtLastSave))
		{
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}
		if (InTransactionEventId > HeadTransactionEventIdAtLastSave)
		{
			OutIsLive = true;
			return ESQLitePreparedStatementExecuteRowResult::Stop;
		}
		return ESQLitePreparedStatementExecuteRowResult::Continue;
	});
}

bool FConcertSyncSessionDatabase::GetLiveTransactionEventIds(TArray<int64>& OutTransactionEventIds) const
{
	OutTransactionEventIds.Reset();

	bool bResult = false;
	{
		TSet<int64> TransactionEventIdsSet;
		bResult = Statements->GetPackageNameIdsWithTransactions([this, &TransactionEventIdsSet](const int64 InPackageNameId)
		{
			const bool bInnerResult = EnumerateLiveTransactionEventIdsForPackage(InPackageNameId, [&TransactionEventIdsSet](const int64 InTransactionEventId)
			{
				TransactionEventIdsSet.Add(InTransactionEventId);
				return true;
			});
			return bInnerResult
				? ESQLitePreparedStatementExecuteRowResult::Continue
				: ESQLitePreparedStatementExecuteRowResult::Error;
		});

		if (bResult)
		{
			OutTransactionEventIds = TransactionEventIdsSet.Array();
			OutTransactionEventIds.Sort();
		}
	}
	return bResult;
}

bool FConcertSyncSessionDatabase::GetLiveTransactionEventIdsForPackage(const FName InPackageName, TArray<int64>& OutTransactionEventIds) const
{
	OutTransactionEventIds.Reset();
	return EnumerateLiveTransactionEventIdsForPackage(InPackageName, [&OutTransactionEventIds](const int64 InTransactionEventId)
	{
		OutTransactionEventIds.Add(InTransactionEventId);
		return true;
	});
}

bool FConcertSyncSessionDatabase::PackageHasLiveTransactions(const FName InPackageName, bool& OutHasLiveTransaction) const
{
	OutHasLiveTransaction = false;
	return EnumerateLiveTransactionEventIdsForPackage(InPackageName, [&OutHasLiveTransaction](const int64)
	{
		OutHasLiveTransaction = true;
		return false;
	});
}

bool FConcertSyncSessionDatabase::EnumerateLiveTransactionEventIdsForPackage(const FName InPackageName, TFunctionRef<bool(int64)> InCallback) const
{
	int64 PackageNameId = 0;
	if (!GetPackageNameId(InPackageName, PackageNameId))
	{
		// If the package name isn't mapped in the database, then there's no transactions for this package
		return true;
	}
	return EnumerateLiveTransactionEventIdsForPackage(PackageNameId, InCallback);
}

bool FConcertSyncSessionDatabase::EnumeratePackageNamesWithLiveTransactions(TFunctionRef<bool(FName)> InCallback) const
{
	return Statements->GetPackageNameIdsMaxTransactionId([this, &InCallback](const int64 InPackageNameId, const int64 InMaxTransactionEventId)
	{
		// Get the transaction id at last save, if the max transaction id for a package name id is greater than its transaction id at last save, it has live transactions
		int64 HeadTransactionEventIdAtLastSave = 0;
		Statements->GetUnmutedPackageTransactionEventIdAtLastSave(InPackageNameId, HeadTransactionEventIdAtLastSave);
		
		if (InMaxTransactionEventId > HeadTransactionEventIdAtLastSave)
		{
			FName PackageName;
			if (GetPackageName(InPackageNameId, PackageName))
			{
				return InCallback(PackageName)
					? ESQLitePreparedStatementExecuteRowResult::Continue
					: ESQLitePreparedStatementExecuteRowResult::Stop;
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}
		return ESQLitePreparedStatementExecuteRowResult::Continue;
	});
}

bool FConcertSyncSessionDatabase::EnumerateLiveTransactionEventIdsForPackage(const int64 InPackageNameId, TFunctionRef<bool(int64)> InCallback) const
{
	int64 HeadTransactionEventIdAtLastSave = 0;
	if (!Statements->GetUnmutedPackageTransactionEventIdAtLastSave(InPackageNameId, HeadTransactionEventIdAtLastSave) || HeadTransactionEventIdAtLastSave == MAX_int64)
	{
		return false;
	}

	return Statements->GetTransactionEventIdsInRangeForPackageNameId(InPackageNameId, HeadTransactionEventIdAtLastSave + 1, [&InCallback](const int64 InTransactionEventId)
	{
		return InCallback(InTransactionEventId)
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});
}

bool FConcertSyncSessionDatabase::AddDummyPackageEvent(const FName InPackageName, int64& OutPackageEventId)
{
	auto AddDummyPackageEventImpl = [this, InPackageName, &OutPackageEventId]()
	{
		// Find the head package event info
		{
			int64 PackageNameId = 0;
			if (GetPackageNameId(InPackageName, PackageNameId) && Statements->GetPackageHeadEventId(PackageNameId, OutPackageEventId) && OutPackageEventId > 0)
			{
				// If the head package event is a dummy event with no activity associated with it, we'll re-use the head package event, otherwise we'll add a new one
				FString FoundDataFilename;
				FConcertPackageInfo PackageInfo;
				int64 PackageRevision;
				if (Statements->GetPackageEventForId(OutPackageEventId, PackageRevision, PackageInfo, FoundDataFilename))
				{
					if (PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Dummy)
					{
						// Does this package have associated activity? If so, we need to keep it as-is
						FConcertSyncActivity FoundActivity;
						if (!GetActivityForEvent(OutPackageEventId, EConcertSyncActivityEventType::Package, FoundActivity))
						{
							// Update this package event
							return Statements->GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave)
								&& Statements->SetPackageEvent(OutPackageEventId, PackageNameId, PackageRevision, PackageInfo.TransactionEventIdAtSave, PackageInfo, FoundDataFilename);
						}
					}
				}
			}
		}

		// Add a new package event
		FConcertPackageInfo DummyPackageInfo;
		FConcertPackageDataStream DummyPackageDataStream;
		DummyPackageInfo.PackageName = InPackageName;
		DummyPackageInfo.PackageUpdateType = EConcertPackageUpdateType::Dummy;
		return Statements->GetTransactionMaxEventId(DummyPackageInfo.TransactionEventIdAtSave)
			&& AddPackageEvent(DummyPackageInfo, DummyPackageDataStream, OutPackageEventId);
	};

	FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
	return ScopedTransaction.CommitOrRollback(AddDummyPackageEventImpl());
}

bool FConcertSyncSessionDatabase::AddPackageEvent(const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream, int64& OutPackageEventId)
{
	// Get the next package ID
	if (!GetPackageMaxEventId(OutPackageEventId) || OutPackageEventId == MAX_int64)
	{
		return false;
	}
	++OutPackageEventId;

	// Get the next package revision
	int64 PackageRevision = 0;
	if (!GetPackageHeadRevision(InPackageInfo.PackageName, PackageRevision) || PackageRevision == MAX_int64)
	{
		return false;
	}
	++PackageRevision;

	return SetPackageEvent(OutPackageEventId, PackageRevision, InPackageInfo, &InPackageDataStream);
}

bool FConcertSyncSessionDatabase::UpdatePackageEvent(const int64 InPackageEventId, FConcertSyncPackageEventData& PackageEventSrc)
{
	int64 MaxPackageEventId;
	if (GetPackageMaxEventId(MaxPackageEventId) && InPackageEventId <= MaxPackageEventId) // Ensure the package ID is in bound.
	{
		FConcertSyncSessionDatabaseScopedTransaction ScopedTransaction(*Statements);
		return ScopedTransaction.CommitOrRollback(
			SetPackageEvent(InPackageEventId, PackageEventSrc.MetaData.PackageRevision, PackageEventSrc.MetaData.PackageInfo, &PackageEventSrc.PackageDataStream)
		);
	}

	return false;
}

bool FConcertSyncSessionDatabase::SetPackageEvent(const int64 InPackageEventId, int64 InPackageRevision, const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream* InPackageDataStream)
{
	if (!ensureAlwaysMsgf(InPackageRevision > 0, TEXT("Invalid package revision! Must be greater than zero.")))
	{
		return false;
	}
	if (!ensureAlwaysMsgf(!InPackageInfo.PackageName.IsNone(), TEXT("Invalid package name! Must be set.")))
	{
		return false;
	}

	// Ensure an entry for this package name
	int64 PackageNameId = 0;
	if (!EnsurePackageNameId(InPackageInfo.PackageName, PackageNameId))
	{
		return false;
	}

	// Write the data blob file
	const FString PackageDataFilename = PackageDataUtil::GetDataFilename(InPackageInfo.PackageName, InPackageRevision);

	if (InPackageDataStream) // If !MetaDataOnly
	{
		const FString DstPackagBlobPathname = PackageDataUtil::GetDataPath(SessionPath) / PackageDataFilename;
		if (!SavePackage(DstPackagBlobPathname, InPackageInfo, *InPackageDataStream))
		{
			return false;
		}
	}

	// Add the database entry
	return Statements->SetPackageEvent(InPackageEventId, PackageNameId, InPackageRevision, InPackageInfo.TransactionEventIdAtSave, InPackageInfo, PackageDataFilename);
}

bool FConcertSyncSessionDatabase::AddReplicationEvent(const FConcertSyncReplicationEvent& InReplicationEvent, int64& OutReplicationEventId)
{
	return Statements->AddReplicationData(InReplicationEvent, OutReplicationEventId);
}

bool FConcertSyncSessionDatabase::SetReplicationEvent(const int64 InReplicationEventId, const FConcertSyncReplicationEvent& InReplicationEvent)
{
	return Statements->SetReplicationData(InReplicationEventId, InReplicationEvent);
}

bool FConcertSyncSessionDatabase::GetPackageEventMetaData(const int64 InPackageEventId, int64& OuptPackageRevision, FConcertPackageInfo& OutPackageInfo) const
{
	FString UnusedDataFilename;
	return Statements->GetPackageEventForId(InPackageEventId, OuptPackageRevision, OutPackageInfo, UnusedDataFilename);
}

bool FConcertSyncSessionDatabase::GetPackageHeadEventFromName(const FName InPackageName, int64& OutPackageEventId)
{
	int64 PackageNameId = 0;
	if (!GetPackageNameId(InPackageName, PackageNameId))
	{
		return false;
	}

	return Statements->GetPackageHeadEventId(PackageNameId, OutPackageEventId);
}

bool FConcertSyncSessionDatabase::GetPackageEvent(const int64 InPackageEventId, const TFunctionRef<void(FConcertSyncPackageEventData&)>& PackageEventFn) const
{
	FString DataFilename;
	FConcertSyncPackageEventData PackageEventSrc;
	if (Statements->GetPackageEventForId(InPackageEventId, PackageEventSrc.MetaData.PackageRevision, PackageEventSrc.MetaData.PackageInfo, DataFilename))
	{
		const FString PackageDataPathname = PackageDataUtil::GetDataPath(SessionPath) / DataFilename;
		return LoadPackage(PackageDataPathname, [&PackageEventSrc, &PackageEventFn](FConcertPackageDataStream& InPackageDataStream)
		{
			PackageEventSrc.PackageDataStream = InPackageDataStream;
			PackageEventFn(PackageEventSrc);
		});
	}

	return false;
}

bool FConcertSyncSessionDatabase::EnumeratePackageNamesWithHeadRevision(TFunctionRef<bool(FName)> InCallback, bool IgnorePersisted) const
{
	// if we ignore packages with persist event we need to compare head revision against entry in the persist table.
	if (IgnorePersisted)
	{
		return Statements->GetMaxPackageEventIdAndTransactionEventIdAtSavePerPackageNameId([this, &InCallback](int64 InPackageNameId, int64 InMaxPackageEventId, int64 TransactionEventIdAtSave)
		{
			// We enumerate the packages if there isn't an entry in persist events with this MaxPackageEventId or 
			// if the TransactionEventIdAtPersist and TransactionEventIdAtSave doesn't match (in case a dummy event got squashed)
			int64 PersistEventId = 0, TransactionEventIdAtPersist = 0;
			if (!Statements->GetPersistEventId(InMaxPackageEventId, PersistEventId, TransactionEventIdAtPersist)
				|| TransactionEventIdAtPersist != TransactionEventIdAtSave)
			{
				FName PackageName;
				if (GetPackageName(InPackageNameId, PackageName))
				{
					return InCallback(PackageName)
						? ESQLitePreparedStatementExecuteRowResult::Continue
						: ESQLitePreparedStatementExecuteRowResult::Stop;
				}
				return ESQLitePreparedStatementExecuteRowResult::Error;
			}
			return ESQLitePreparedStatementExecuteRowResult::Continue;
		});
	}
	
	// otherwise we can just gather distinct packages in the package events table
	return Statements->GetUniquePackageNameIdsForPackageEvents([this, &InCallback](int64 InPackageNameId)
	{
		FName PackageName;
		if (GetPackageName(InPackageNameId, PackageName))
		{
			return InCallback(PackageName)
				? ESQLitePreparedStatementExecuteRowResult::Continue
				: ESQLitePreparedStatementExecuteRowResult::Stop;
		}
		return ESQLitePreparedStatementExecuteRowResult::Error;
	});
}

bool FConcertSyncSessionDatabase::EnumerateHeadRevisionPackageData(TFunctionRef<bool(const FConcertPackageInfo&, FConcertPackageDataStream&)> InCallback) const
{
	return Statements->GetUniquePackageNameIdsForPackageEvents([this, &InCallback](int64 InPackageNameId)
	{
		int64 PackageHeadRevision = 0;
		if (Statements->GetUnmutedPackageHeadRevision(InPackageNameId, PackageHeadRevision))
		{
			FString DataFilename;
			FConcertPackageInfo PackageInfo;
			if (Statements->GetPackageDataForRevision(InPackageNameId, PackageHeadRevision, PackageInfo, DataFilename))
			{
				bool bContinue = true;
				const FString PackageBlobPathname = PackageDataUtil::GetDataPath(SessionPath) / DataFilename;
				if (LoadPackage(PackageBlobPathname, [&bContinue, &InCallback, &PackageInfo](FConcertPackageDataStream& InPackageDataStream){ bContinue = InCallback(PackageInfo, InPackageDataStream); }))
				{
					return bContinue ? ESQLitePreparedStatementExecuteRowResult::Continue : ESQLitePreparedStatementExecuteRowResult::Stop;
				}
			}
		}
		return ESQLitePreparedStatementExecuteRowResult::Error;
	});
}

bool FConcertSyncSessionDatabase::GetPackageMaxEventId(int64& OutPackageEventId) const
{
	return Statements->GetPackageMaxEventId(OutPackageEventId);
}

bool FConcertSyncSessionDatabase::AddPersistEventForHeadRevision(FName InPackageName, int64& OutPersistEventId)
{
	int64 PackageNameId = 0, HeadPackageEventId = 0, TransactionEventIdAtSave = 0;
	if (GetPackageNameId(InPackageName, PackageNameId) 
		&& Statements->GetPackageHeadEventIdAndTransactionIdAtSave(PackageNameId, HeadPackageEventId, TransactionEventIdAtSave)
		&& HeadPackageEventId > 0)
	{
		Statements->AddPersistEvent(HeadPackageEventId, TransactionEventIdAtSave, OutPersistEventId);
	}
	return false;
}

bool FConcertSyncSessionDatabase::GetPackageInfoForRevision(const FName InPackageName, FConcertPackageInfo& OutPackageInfo, const int64* InPackageRevision) const
{
	const TOptional<int64> PackageRevision = GetSpecifiedOrHeadPackageRevision(InPackageName, InPackageRevision);
	if (!PackageRevision)
	{
		return false;
	}

	int64 PackageNameId = 0;
	if (!GetPackageNameId(InPackageName, PackageNameId))
	{
		return false;
	}

	FString DataFilename;
	return Statements->GetPackageDataForRevision(PackageNameId, *PackageRevision, OutPackageInfo, DataFilename);
}

bool FConcertSyncSessionDatabase::GetPackageDataForRevision(const FName InPackageName, const TFunctionRef<void(const FConcertPackageInfo&, FConcertPackageDataStream&)>& InPackageDataCallback, const int64* InPackageRevision) const
{
	return HandleRequestPackageRequest(InPackageName, InPackageRevision, [this, InPackageDataCallback](FConcertPackageInfo PackageInfo, FString DataFilename)
	{
		const FString PackageDataPathname = PackageDataUtil::GetDataPath(SessionPath) / DataFilename;
		return LoadPackage(PackageDataPathname, [&PackageInfo, InPackageDataCallback](FConcertPackageDataStream& InPackageDataStream)
		{
			InPackageDataCallback(PackageInfo, InPackageDataStream);
		});
	});
}

TOptional<int64> FConcertSyncSessionDatabase::GetPackageSizeForRevision(const FName InPackageName, const int64* InPackageRevision) const
{
	TOptional<int64> Result;
	HandleRequestPackageRequest(InPackageName, InPackageRevision, [this, &Result](FConcertPackageInfo PackageInfo, FString DataFilename)
	{
		const FString PackageDataPathname = PackageDataUtil::GetDataPath(SessionPath) / DataFilename;
		const int64 FileSize = IFileManager::Get().FileSize(*PackageDataPathname);
		if (FileSize >= 0)
		{
			Result = FileSize;
		}
		return true;
	});
	return Result;
}

bool FConcertSyncSessionDatabase::GetPackageHeadRevision(const FName InPackageName, int64& OutRevision) const
{
	int64 PackageNameId = 0;
	if (!GetPackageNameId(InPackageName, PackageNameId))
	{
		// If the package name isn't mapped in the database, then there's no history for this package which means it's at revision zero
		OutRevision = 0;
		return true;
	}

	return Statements->GetUnmutedPackageHeadRevision(PackageNameId, OutRevision);
}

bool FConcertSyncSessionDatabase::IsHeadRevisionPackageEvent(const int64 InPackageEventId, bool& OutIsHeadRevision) const
{
	int64 PackageNameId = 0;
	int64 PackageRevision = 0;
	int64 PackageHeadRevision = 0;
	if (Statements->GetPackageNameIdAndRevisonForId(InPackageEventId, PackageNameId, PackageRevision) && Statements->GetUnmutedPackageHeadRevision(PackageNameId, PackageHeadRevision))
	{
		OutIsHeadRevision = PackageRevision == PackageHeadRevision;
		return true;
	}
	return false;
}

bool FConcertSyncSessionDatabase::GetReplicationMaxEventIdByClientAndType(const FGuid& EndpointId, EConcertSyncReplicationActivityType ActivityType, int64& OutReplicationEventId) const
{
	return Statements->GetReplicationMaxEventIdByClientAndType(EndpointId, ActivityType, OutReplicationEventId);
}

bool FConcertSyncSessionDatabase::GetObjectPathName(const int64 InObjectNameId, FName& OutObjectPathName) const
{
	return Statements->GetObjectPathName(InObjectNameId, OutObjectPathName);
}

bool FConcertSyncSessionDatabase::GetObjectNameId(const FName InObjectPathName, int64& OutObjectNameId) const
{
	return Statements->GetObjectNameId(InObjectPathName, OutObjectNameId);
}

bool FConcertSyncSessionDatabase::EnsureObjectNameId(const FName InObjectPathName, int64& OutObjectNameId)
{
	return GetObjectNameId(InObjectPathName, OutObjectNameId)
		|| Statements->AddObjectPathName(InObjectPathName, OutObjectNameId);
}

bool FConcertSyncSessionDatabase::GetPackageName(const int64 InPackageNameId, FName& OutPackageName) const
{
	return Statements->GetPackageName(InPackageNameId, OutPackageName);
}

bool FConcertSyncSessionDatabase::GetPackageNameId(const FName InPackageName, int64& OutPackageNameId) const
{
	return Statements->GetPackageNameId(InPackageName, OutPackageNameId);
}

bool FConcertSyncSessionDatabase::EnsurePackageNameId(const FName InPackageName, int64& OutPackageNameId)
{
	return GetPackageNameId(InPackageName, OutPackageNameId)
		|| Statements->AddPackageName(InPackageName, OutPackageNameId);
}

bool FConcertSyncSessionDatabase::MapResourceNamesForLock(const int64 InLockEventId, const TArray<FName>& InResourceNames)
{
	bool bResult = Statements->UnmapObjectNameIdsForLockEventId(InLockEventId);
	for (const FName& ResourceName : InResourceNames)
	{
		int64 ObjectNameId = 0;
		bResult &= EnsureObjectNameId(ResourceName, ObjectNameId) && Statements->MapObjectNameIdToLockEventId(ObjectNameId, InLockEventId);
	}
	return bResult;
}

bool FConcertSyncSessionDatabase::MapPackageNamesForTransaction(const int64 InTransactionEventId, const FConcertTransactionFinalizedEvent& InTransactionEvent)
{
	bool bResult = Statements->UnmapPackageNameIdsForTransactionEventId(InTransactionEventId);
	for (const FName& PackageName : InTransactionEvent.ModifiedPackages)
	{
		int64 PackageNameId = 0;
		bResult &= EnsurePackageNameId(PackageName, PackageNameId) && Statements->MapPackageNameIdToTransactionEventId(PackageNameId, InTransactionEventId);
	}
	return bResult;
}

bool FConcertSyncSessionDatabase::MapObjectNamesForTransaction(const int64 InTransactionEventId, const FConcertTransactionFinalizedEvent& InTransactionEvent)
{
	bool bResult = Statements->UnmapObjectNameIdsForTransactionEventId(InTransactionEventId);
	for (const FConcertExportedObject& Object : InTransactionEvent.ExportedObjects)
	{
		// TODO: This isn't always the correct way to build the object path (re: SUBOBJECT_DELIMITER)
		const FName ObjectPathName = *FString::Printf(TEXT("%s.%s"), *Object.ObjectId.ObjectOuterPathName.ToString(), *Object.ObjectId.ObjectName.ToString());

		int64 ObjectNameId = 0;
		bResult &= EnsureObjectNameId(ObjectPathName, ObjectNameId) && Statements->MapObjectNameIdToTransactionEventId(ObjectNameId, InTransactionEventId);
	}
	return bResult;
}

bool FConcertSyncSessionDatabase::SaveTransaction(const FString& InTransactionFilename, const FStructOnScope& InTransaction) const
{
	TArray<uint8> SerializedTransactionData;
	return TransactionDataUtil::WriteTransaction(InTransaction, SerializedTransactionData) && TransactionFileCache->SaveAndCacheFile(InTransactionFilename, MoveTemp(SerializedTransactionData));
}

bool FConcertSyncSessionDatabase::LoadTransaction(const FString& InTransactionFilename, FStructOnScope& OutTransaction) const
{
	TArray<uint8> SerializedTransactionData;
	if (TransactionFileCache->FindOrCacheFile(InTransactionFilename, SerializedTransactionData) && TransactionDataUtil::ReadTransaction(SerializedTransactionData, OutTransaction))
	{
		if (ensureAlwaysMsgf(OutTransaction.GetStruct()->IsChildOf(FConcertTransactionEventBase::StaticStruct()), TEXT("LoadTransaction can only be used with types deriving from FConcertTransactionEventBase")))
		{
			return true;
		}
	}
	return false;
}

bool HandleWritePackageResult(const FName& PackageName, const FString& InDstPackageBlobPathname, TUniquePtr<FConcertFileCache>& Cache, PackageDataUtil::WritePackageResult Result)
{
	auto SaveResult = [](const FString& InDstPackageBlobPathname, TUniquePtr<FConcertFileCache>& Cache, PackageDataUtil::WritePackageResult Result)
	{
		if (Result.CacheData.Num()>0)
		{
			if (Result.bShouldCache)
			{
				return Cache->SaveAndCacheFile(InDstPackageBlobPathname, MoveTemp(Result.CacheData));
			}
			TUniquePtr<FArchive> DstAr(IFileManager::Get().CreateFileWriter(*InDstPackageBlobPathname));
			if (DstAr)
			{
				DstAr->Serialize(Result.CacheData.GetData(), Result.CacheData.Num());
				return !DstAr->IsError();
			}
			UE_LOGF(LogConcert, Warning, "Failed to handle package write result of file '%ls'.", *InDstPackageBlobPathname);
			return false;
		}
		return true;
	};

	const bool bSaveResult = SaveResult(InDstPackageBlobPathname, Cache, Result);
	if (bSaveResult)
	{
		UE::ConcertSyncCore::SyncDatabase::GetOnPackageSavedDelegate().Broadcast(PackageName);
	}

	return bSaveResult;
}

bool FConcertSyncSessionDatabase::SavePackage(const FString& InDstPackageBlobPathname, const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream)
{
	SCOPED_CONCERT_TRACE(FConcertSyncSessionDatabase_SavePackage);

	// There are a few different cases that affect how SavePackage works.
	//  1. In memory binary data:  We have received a package of data from that is in memory.  In this case the archive
	//     provided is Null and we can simply copy that data into a temporary buffer and perform an asynchronous write
	//     to disk.
	//  2. No in memory: We have received a handle to a file on disk via InPackageDataStream.DstAr.  We cannot guarantee the lifetime
	//     of this object and thus we cannot perform asynchronous work (Unless the raw data via DataBlob has already
	//     been provided) in which case we ignore the DstAr and do asynchronous path.
	//
	FName PackageName = InPackageInfo.PackageUpdateType == EConcertPackageUpdateType::Renamed ? InPackageInfo.NewPackageName : InPackageInfo.PackageName;
	if (InPackageDataStream.DataBlob && InPackageDataStream.DataBlob->Num() > 0)
	{
		ScheduleAsyncWrite(PackageName, InDstPackageBlobPathname, InPackageDataStream);
		return true;
	}
	else
	{
		if (PackageDataUtil::WillUseMemoryWriter(InPackageDataStream.DataSize))
		{
			FArchive NullArchive;
			PackageDataUtil::WritePackageResult Result = PackageDataUtil::WritePackage(InPackageDataStream, &NullArchive);
			return HandleWritePackageResult(PackageName, *InDstPackageBlobPathname, PackageFileCache, MoveTemp(Result));
		}
		else
		{
			TUniquePtr<FArchive> DstAr(IFileManager::Get().CreateFileWriter(*InDstPackageBlobPathname));
			if (DstAr)
			{
				PackageDataUtil::WritePackage(InPackageDataStream, DstAr.Get());
				return !DstAr->IsError();
			}
		}
	}

	return false;
}

bool FConcertSyncSessionDatabase::LoadPackage(const FString& InPackageBlobFilename, const TFunctionRef<void(FConcertPackageDataStream&)>& PackageDataStreamFn) const
{
	// First check the deferred file IO buffer before looking on disk.
	if(FConcertPackageAsyncDataStreamPtr const* Found = DeferredLargePackageIOPtr->GetMap().Find(InPackageBlobFilename))
	{
		FConcertPackageAsyncDataStreamPtr Owned = *Found;
		FConcertPackageDataStream Stream = Owned->PackageStream;
		FMemorySyncDbReader PackageBlobAr(Owned->PackageData);
		Stream.DataAr = &PackageBlobAr;
		PackageDataStreamFn(Stream);
		return true;
	}

	int64 PackageBlobSize = IFileManager::Get().FileSize(*InPackageBlobFilename);
	if (PackageBlobSize < 0) // Possible if the file doesn't exist.
	{
		UE_LOGF(LogConcert, Warning, "Failed to read the size of database file '%ls'.", *InPackageBlobFilename);
		return false;
	}

	 // Try the cache if the blob is small enough to be found or stored.
	if (PackageDataUtil::ShouldCachePackageBlob(static_cast<uint64>(PackageBlobSize)))
	{
		TArray<uint8> PackageBlob;
		return PackageFileCache->FindOrCacheFile(InPackageBlobFilename, PackageBlob) && PackageDataUtil::ExtractPackageData(PackageBlob, PackageDataStreamFn);
	}

	return PackageDataUtil::ExtractPackageData(InPackageBlobFilename, PackageDataStreamFn);
}

void FConcertSyncSessionDatabase::FlushAsynchronousTasks()
{
	while (!DeferredLargePackageIOPtr->GetMap().IsEmpty())
	{
		UpdateAsynchronousTasks();
		const float SleepTimeInSeconds = 0.010f;
		FPlatformProcess::Sleep(SleepTimeInSeconds);
	}
}

bool FConcertSyncSessionDatabase::HasWritePackageTasksCompleted() const
{
	return DeferredLargePackageIOPtr->GetMap().IsEmpty();
}

void FConcertSyncSessionDatabase::UpdateAsynchronousTasks()
{
	for (auto It = DeferredLargePackageIOPtr->GetMap().CreateIterator(); It; ++It)
	{
		if (It->Value->AsyncTask.IsCompleted())
		{
			if (It->Value->AsyncTask.GetResult())
			{
				HandleWritePackageResult(It->Value->CachedPackageName, It->Key, PackageFileCache, MoveTemp(It->Value->Result));
			}
			else
			{
				UE_LOGF(LogConcert, Error, "Async task failed to write package %ls", *It->Value->CachedPackageName.ToString());
			}
			It.RemoveCurrent();
		}
	}
}

bool FConcertSyncSessionDatabase::HandleRequestPackageRequest(const FName InPackageName, const int64* InPackageRevision, FProcessPackageRequest HandleFunc) const
{
	int64 PackageNameId = 0;
	FConcertPackageInfo PackageInfo;
	FString DataFilename;
	
	const TOptional<int64> PackageRevision = GetSpecifiedOrHeadPackageRevision(InPackageName, InPackageRevision);
	if (PackageRevision
		&& GetPackageNameId(InPackageName, PackageNameId)
		&& Statements->GetPackageDataForRevision(PackageNameId, *PackageRevision, PackageInfo, DataFilename))
	{
		return HandleFunc(PackageInfo, DataFilename);
	}
	
	return false;
}

TOptional<int64> FConcertSyncSessionDatabase::GetSpecifiedOrHeadPackageRevision(FName InPackageName, const int64* InPackageRevision) const
{
	int64 PackageRevision = 0;
	if (InPackageRevision)
	{
		PackageRevision = *InPackageRevision;
	}
	else if (!GetPackageHeadRevision(InPackageName, PackageRevision))
	{
		return {};
	}
	
	if (PackageRevision == 0)
	{
		return {};
	}
	return PackageRevision;
}

void FConcertSyncSessionDatabase::ScheduleAsyncWrite(const FName& InPackageName, const FString& InDstPackageBlobPathname, FConcertPackageDataStream& InPackageDataStream)
{
	check(InPackageDataStream.DataBlob && InPackageDataStream.DataBlob->Num() > 0);

	FConcertPackageAsyncDataStreamPtr SharedStream = MakeShared<FConcertPackageAsyncDataStream,ESPMode::ThreadSafe>(InPackageDataStream, InPackageName);
	if(FConcertPackageAsyncDataStreamPtr* Item = DeferredLargePackageIOPtr->GetMap().Find(InDstPackageBlobPathname))
	{
		if ((*Item)->AsyncTask.GetResult())
		{
			HandleWritePackageResult((*Item)->CachedPackageName, InDstPackageBlobPathname, PackageFileCache, MoveTemp((*Item)->Result));
		}
	}
	DeferredLargePackageIOPtr->GetMap().FindOrAdd(InDstPackageBlobPathname) = SharedStream;
	SharedStream->AsyncTask = UE::Tasks::Launch(TEXT("ConcertWritePackageAsync"), [InDstPackageBlobPathname,SharedStream]()
	{
		TUniquePtr<FArchive> DstAr(IFileManager::Get().CreateFileWriter(*InDstPackageBlobPathname));
		ensureMsgf(DstAr, TEXT("ConcertSyncSessionDatabase: Unable to open a file for writing. Ensure you have enough disk space and long paths are enabled."));

		FConcertPackageDataStream& InPackageDataStream = SharedStream->PackageStream;
		FMemorySyncDbReader PackageBlobAr(SharedStream->PackageData);
		InPackageDataStream.DataAr = &PackageBlobAr;
		SharedStream->Result = PackageDataUtil::WritePackage(InPackageDataStream, DstAr.Get());
		return DstAr && !DstAr->IsError();
	});
}

bool ConcertSyncSessionDatabaseFilterUtil::TransactionEventPassesFilter(const int64 InTransactionEventId, const FConcertSessionFilter& InSessionFilter, const FConcertSyncSessionDatabase& InDatabase)
{
	check(InDatabase.IsValid());

	if (!InSessionFilter.bOnlyLiveData)
	{
		return true;
	}

	bool bIsLive = false;
	InDatabase.IsLiveTransactionEvent(InTransactionEventId, bIsLive);
	return bIsLive;
}

bool ConcertSyncSessionDatabaseFilterUtil::PackageEventPassesFilter(const int64 InPackageEventId, const FConcertSessionFilter& InSessionFilter, const FConcertSyncSessionDatabase& InDatabase)
{
	check(InDatabase.IsValid());

	if (!InSessionFilter.bOnlyLiveData)
	{
		return true;
	}

	bool bIsHeadRevision = false;
	InDatabase.IsHeadRevisionPackageEvent(InPackageEventId, bIsHeadRevision);
	return bIsHeadRevision;
}
