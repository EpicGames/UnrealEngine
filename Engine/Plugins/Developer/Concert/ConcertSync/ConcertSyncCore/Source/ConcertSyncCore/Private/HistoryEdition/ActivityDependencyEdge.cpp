// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "CoreGlobals.h"

FString UE::ConcertSyncCore::LexToString(EActivityDependencyReason Reason)
{
	switch (Reason)
	{
		case EActivityDependencyReason::PackageCreation: return TEXT("PackageCreation");
		case EActivityDependencyReason::PackageRemoval: return TEXT("PackageRemoval");
		case EActivityDependencyReason::PackageRename: return TEXT("PackageRename");
		case EActivityDependencyReason::PackageEdited: return TEXT("PackageEdited");
		case EActivityDependencyReason::EditAfterPreviousPackageEdit: return TEXT("EditPossiblyDependsOnPackage");
		case EActivityDependencyReason::SubobjectCreation: return TEXT("SubobjectCreation");
		case EActivityDependencyReason::SubobjectRemoval: return TEXT("SubobjectRemoval");
		default:
			checkNoEntry();
			static_assert(static_cast<int32>(EActivityDependencyReason::Count) == 7, "Update this location when you change EActivityDependencyReason");
			return TEXT("Invalid");
	}
}
