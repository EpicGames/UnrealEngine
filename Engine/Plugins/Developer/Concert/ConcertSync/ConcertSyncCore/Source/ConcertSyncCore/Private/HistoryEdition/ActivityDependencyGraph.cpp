// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/ActivityDependencyGraph.h"

#include "ConcertLogGlobal.h"
#include "CoreGlobals.h"
#include "HistoryEdition/ActivityDependencyEdge.h"
#include "HistoryEdition/ActivityNode.h"
#include "Templates/Function.h"

#include "Algo/ForEach.h"

UE::ConcertSyncCore::FActivityNodeID UE::ConcertSyncCore::FActivityDependencyGraph::AddActivity(int64 ActivityIndex, EActivityNodeFlags NodeFlags)
{
	const TOptional<FActivityNodeID> PreexistingID = FindNodeByActivity(ActivityIndex);
	if (ensureMsgf(!PreexistingID.IsSet(), TEXT("Activity is already registered!")))
	{
		FActivityNodeID NodeID(Nodes.Num());
		Nodes.Emplace(ActivityIndex, NodeID, NodeFlags);
		return NodeID;
	}

	return *PreexistingID;
}

bool UE::ConcertSyncCore::FActivityDependencyGraph::AddDependency(FActivityNodeID From, FActivityDependencyEdge To)
{
	if (!ensure(Nodes.IsValidIndex(From.ID) && Nodes.IsValidIndex(To.GetDependedOnNodeID().ID)))
	{
		return false;
	}
	
	FActivityNode& FromNode = GetNodeByIdInternal(From);
	FActivityNode& ToNode = GetNodeByIdInternal(To.GetDependedOnNodeID());

	const bool bIsValidDependency = FromNode.GetActivityId() > ToNode.GetActivityId();
	if (bIsValidDependency)
	{
		FromNode.Dependencies.AddUnique(To);
		ToNode.AffectedChildren.AddUnique(From);
		return true;
	}
	
	UE_CLOGF(!bIsValidDependency, LogConcert, Error, "Activities can only depend on preceding activities!");
	return false;
}

TOptional<UE::ConcertSyncCore::FActivityNodeID> UE::ConcertSyncCore::FActivityDependencyGraph::FindNodeByActivity(int64 ActivityID) const
{
	for (int32 i = 0; i < Nodes.Num(); ++i)
	{
		if (Nodes[i].ActivityId == ActivityID)
		{
			return FActivityNodeID(i);
		}
	}

	return {};
}

const UE::ConcertSyncCore::FActivityNode& UE::ConcertSyncCore::FActivityDependencyGraph::GetNodeById(FActivityNodeID ID) const
{
	const size_t Index = static_cast<size_t>(ID);
	check(Nodes.IsValidIndex(Index));
	return Nodes[Index];
}

UE::ConcertSyncCore::FActivityNode& UE::ConcertSyncCore::FActivityDependencyGraph::GetNodeByIdInternal(FActivityNodeID ID)
{
	const size_t Index = static_cast<size_t>(ID);
	check(Nodes.IsValidIndex(Index));
	return Nodes[Index];
}

void UE::ConcertSyncCore::FActivityDependencyGraph::ForEachNode(TFunctionRef<void(const FActivityNode&)> ConsumerFunc) const
{
	Algo::ForEach(Nodes, ConsumerFunc);
}
