// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/ActivityNode.h"

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "HistoryEdition/ActivityDependencyGraph.h"

bool UE::ConcertSyncCore::FActivityNode::DependsOnNode(FActivityNodeID NodeId, TOptional<EActivityDependencyReason> WithReason, TOptional<EDependencyStrength> WithStrength) const
{
	return Dependencies.ContainsByPredicate([NodeId, WithReason, WithStrength](const FActivityDependencyEdge& Edge)
	{
		return Edge.GetDependedOnNodeID() == NodeId
			&& (!WithReason.IsSet() || *WithReason == Edge.GetDependencyReason())
			&& (!WithStrength.IsSet() || *WithStrength == Edge.GetDependencyStrength());
	});
}

bool UE::ConcertSyncCore::FActivityNode::DependsOnActivity(int64 InActivityId, const FActivityDependencyGraph& Graph, TOptional<EActivityDependencyReason> WithReason, TOptional<EDependencyStrength> WithStrength) const
{
	const TOptional<FActivityNodeID> ActivityNodeId = Graph.FindNodeByActivity(InActivityId);
	return ActivityNodeId && Dependencies.ContainsByPredicate([ActivityNodeId, WithReason, WithStrength](const FActivityDependencyEdge& Edge)
	{
		return Edge.GetDependedOnNodeID() == *ActivityNodeId
			&& (!WithReason.IsSet() || *WithReason == Edge.GetDependencyReason())
			&& (!WithStrength.IsSet() || *WithStrength == Edge.GetDependencyStrength());
	});
}

bool UE::ConcertSyncCore::FActivityNode::AffectsActivity(int64 InActivityId, const FActivityDependencyGraph& Graph) const
{
	const TOptional<FActivityNodeID> ActivityNodeId = Graph.FindNodeByActivity(InActivityId);
	return ActivityNodeId && AffectedChildren.Contains(*ActivityNodeId);
}
