// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/DebugDependencyGraph.h"

#include "ConcertSyncSessionTypes.h"
#include "HistoryEdition/ActivityNode.h"
#include "HistoryEdition/ActivityDependencyEdge.h"

#include "ConcertLogGlobal.h"
#include "ConcertSyncSessionDatabase.h"
#include "HistoryEdition/ActivityDependencyGraph.h"

void UE::ConcertSyncCore::Graphviz::MakeNodeTitle(FGraphStringBuilder& WriteTo, FActivityNodeID ToStringify, const FActivityDependencyGraph& Graph, const FConcertSyncSessionDatabase& SessionDatabase, ENodeTitleFlags NodeTitleFlags)
{
	const FActivityNode& Node = Graph.GetNodeById(ToStringify);
	FConcertSyncActivity Activity;
	SessionDatabase.GetActivity(Node.GetActivityId(), Activity);
		
	if ((NodeTitleFlags & ENodeTitleFlags::Summary) != ENodeTitleFlags::None)
	{
		FConcertSyncTransactionEvent TransactionEvent;
		switch (Activity.EventType)
		{
		case EConcertSyncActivityEventType::Transaction:
			if (SessionDatabase.GetTransactionEvent(Activity.EventId, TransactionEvent))
			{
				WriteTo << FConcertSyncTransactionActivitySummary::CreateSummaryForEvent(TransactionEvent).ToDisplayText(FText::GetEmpty()).ToString();
			}
			break;
		case EConcertSyncActivityEventType::Package:
			SessionDatabase.GetPackageEvent(Activity.EventId, [&WriteTo](FConcertSyncPackageEventData& Event)
			{
				WriteTo << FConcertSyncPackageActivitySummary::CreateSummaryForEvent(Event.MetaData.PackageInfo).ToDisplayText(FText::GetEmpty()).ToString();
			});
			break;

			// Dependency graph ignores ignores these
			case EConcertSyncActivityEventType::None:
			case EConcertSyncActivityEventType::Connection:
			case EConcertSyncActivityEventType::Lock:
			default:
				checkNoEntry();
			break;
		}
	}

	if ((NodeTitleFlags & ENodeTitleFlags::ActivityID) != ENodeTitleFlags::None)
	{
		WriteTo << FString::Printf(TEXT(" (%lld)"), Activity.ActivityId);
	}
	return;
}

void UE::ConcertSyncCore::Graphviz::GetNodeStyle(FGraphStringBuilder& WriteTo, FActivityNodeID ToStringify, const FActivityDependencyGraph& Graph, const FConcertSyncSessionDatabase& SessionDatabase)
{
	// https://graphviz.org/doc/info/shapes.html
	const TMap<EConcertSyncActivityEventType, FString> Shapes {
		{ EConcertSyncActivityEventType::None, TEXT("plaintext") },
		{ EConcertSyncActivityEventType::Connection, TEXT("plaintext") },
		{ EConcertSyncActivityEventType::Lock, TEXT("plaintext") },
		{ EConcertSyncActivityEventType::Transaction, TEXT("ellipse") },
		{ EConcertSyncActivityEventType::Package, TEXT("folder") }
	};
	const TMap<EConcertPackageUpdateType, FColor> PackageColours {
			{ EConcertPackageUpdateType::Dummy,  FColor::White },
			{ EConcertPackageUpdateType::Added, FColor(FColor::Green.R * 0.6, FColor::Green.G * 0.6, FColor::Green.B * 0.6) },
			{ EConcertPackageUpdateType::Saved,  FColor::Green },
			{ EConcertPackageUpdateType::Renamed, FColor(FColor::Orange.R * 0.8, FColor::Orange.G * 0.8, FColor::Orange.B * 0.8) },
			{ EConcertPackageUpdateType::Deleted, FColor(FColor::Red.R * 0.8, FColor::Red.G * 0.8, FColor::Red.B * 0.8) }
	};

	const FActivityNode& Node = Graph.GetNodeById(ToStringify);
	FConcertSyncActivity Activity;
	if (!SessionDatabase.GetActivity(Node.GetActivityId(), Activity))
	{
		return;
	}

	const FString Shape = Shapes[Activity.EventType];
	FString ColorHex;

	if (Activity.EventType == EConcertSyncActivityEventType::Package)
	{
		SessionDatabase.GetPackageEvent(Activity.EventId, [&ColorHex, &PackageColours](FConcertSyncPackageEventData& PackageEventData)
		{
			ColorHex = PackageColours[PackageEventData.MetaData.PackageInfo.PackageUpdateType].ToHex();
		});
	}
	else
	{
		ColorHex = FColor(255 * 0.5f, 255 * 0.5f, 255 * 0.5f).ToHex();
	}

	WriteTo << "[shape=\"" << *Shape << "\" style=\"filled\" fillcolor=\"#" << *ColorHex << "\"]";
}

void UE::ConcertSyncCore::Graphviz::GetEdgeStyle(FGraphStringBuilder& WriteTo, const FActivityDependencyEdge& ToStringify)
{
	const TMap<EDependencyStrength, FString> ArrowTypes {
			{ EDependencyStrength::HardDependency, TEXT("open") },
			{ EDependencyStrength::PossibleDependency, TEXT("halfopen") }
	};
	
	const FString ArrowHead = ArrowTypes[ToStringify.GetDependencyStrength()];
	const FString ReasonAsString = LexToString(ToStringify.GetDependencyReason());
	WriteTo << "[arrowhead=\"" << *ArrowHead << "\" label = \"" << *ReasonAsString << "\"]";
}

FString UE::ConcertSyncCore::Graphviz::ExportToGraphviz(const FActivityDependencyGraph& Graph, const FConcertSyncSessionDatabase& SessionDatabase, ENodeTitleFlags NodeTitleFlags)
{
	UE_CLOGF(NodeTitleFlags == ENodeTitleFlags::None, LogConcertDebug, Warning, "No node title flags specified");

	auto MakeNodeTitle = [&Graph, &SessionDatabase](FGraphStringBuilder& WriteTo, FActivityNodeID ToStringify)
	{
		Graphviz::MakeNodeTitle(WriteTo, ToStringify, Graph, SessionDatabase);
	};
	auto GetNodeStyle = [&Graph, &SessionDatabase](FGraphStringBuilder& WriteTo, FActivityNodeID ToStringify)
	{
		Graphviz::GetNodeStyle(WriteTo, ToStringify, Graph, SessionDatabase);
	};
	auto GetEdgeStyle = [](FGraphStringBuilder& WriteTo, FActivityNodeID EdgeStart, const FActivityDependencyEdge& ToStringify)
	{
		Graphviz::GetEdgeStyle(WriteTo, ToStringify);
	};
	
	return ExportToGraphviz(Graph, MakeNodeTitle, GetNodeStyle, GetEdgeStyle);
}

FString UE::ConcertSyncCore::Graphviz::ExportToGraphviz(const FActivityDependencyGraph& Graph, FMakeNodeTitle MakeNodeTitleFunc, FGetNodeStyle GetNodeStyleFunc, FGetEdgeStyle GetEdgeStyleFunc)
{
	FGraphStringBuilder Result;
	Result << "digraph ActivityDependencyGraph {\n";
	
	TMap<FActivityNodeID, FString> NodeTitles;
	auto GetTitle = [&NodeTitles, MakeNodeTitleFunc](FActivityNodeID NodeID)
	{
		FString* Title = NodeTitles.Find(NodeID);
		if (!Title)
		{
			FGraphStringBuilder TitleBuilder;
			TitleBuilder << "\"";
			MakeNodeTitleFunc(TitleBuilder, NodeID);
			TitleBuilder << "\"";
			Title = &NodeTitles.Add(NodeID, TitleBuilder.ToString());
		};
		return *Title;
	};
	Graph.ForEachNode([&Result, GetTitle, GetNodeStyleFunc, GetEdgeStyleFunc](const FActivityNode& Node)
	{
		const FActivityNodeID NodeID = Node.GetNodeIndex();
		const FString CurrentNodeTitle = *GetTitle(NodeID);
		
		Result << "\t" << *CurrentNodeTitle << " ";
		GetNodeStyleFunc(Result, NodeID);
		Result << "\n";
		
		for (const FActivityDependencyEdge& OutgoingDependency : Node.GetDependencies())
		{
			Result << "\t" << CurrentNodeTitle << " -> " << *GetTitle(OutgoingDependency.GetDependedOnNodeID()) << " ";
			GetEdgeStyleFunc(Result, NodeID, OutgoingDependency);
			Result << "\n"; 
		}
		
		// Make blocks easier to discern for human reader
		Result << "\n";
	});

	Result << TEXT("\n}");
	return Result.ToString();
}
