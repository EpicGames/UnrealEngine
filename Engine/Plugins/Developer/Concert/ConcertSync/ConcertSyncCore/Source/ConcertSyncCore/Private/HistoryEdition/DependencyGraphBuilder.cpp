// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/DependencyGraphBuilder.h"

#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"
#include "HistoryEdition/ActivityDependencyEdge.h"
#include "HistoryEdition/ActivityDependencyGraph.h"
#include "HistoryEdition/ActivityNode.h"

namespace UE::ConcertSyncCore
{
	/** Tracks the activities that last added, saved, removed, edited or modified a package. */
	struct FPackageTracker
	{
		TMap<FName, FActivityID> AddedPackages;
		TMap<FName, FActivityID> SavedPackages;
		TMap<FName, FActivityID> RemovedPackages;
		/** Value: The last activity that renamed a package to the key. */
		TMap<FName, FActivityID> RenamedToPackages;
		/** Value: The last activity that renamed a package from the key to something else. */
		TMap<FName, FActivityID> RenamedFromPackages;
		TMap<FName, FActivityID> ModifiedPackages;
		/** The first activity that modified the given package after a revision of the package has been created. */
		TMap<FName, TArray<FActivityID>> TransactionsAffectingPackageSinceLastPackageActivity;

		enum class ESubobjectState
		{
			Created,
			Removed
		};
		
		/**
		 * Maps FConcertTransactionEventBase::ExportedObject to the last transaction activity
		 * that created it (bAllowCreate == true).
		 */
		TMap<FSoftObjectPath, TPair<FActivityID, ESubobjectState>> CreatedOrRemovedSubobjects;
	};

	class FActivityDependencyGraphBuildAlgorithm
	{
	public:

		static FActivityDependencyGraph BuildGraph(const FConcertSyncSessionDatabase& SessionDatabase);

	private:

		const FConcertSyncSessionDatabase& SessionDatabase;
		
		FActivityDependencyGraph Graph;
		FPackageTracker PackageTracker;

		FActivityDependencyGraphBuildAlgorithm(const FConcertSyncSessionDatabase& SessionDatabase);
		
		bool IsRelevantForDependencies(const FConcertSyncActivity& Activity);
	
		void DiscoverPackageDependencies(const FConcertSyncActivity& Activity);
		void DiscoverPackageDependencies(const FConcertSyncActivity& Activity, const FConcertSyncTransactionEvent& EventData);
		void DiscoverPackageDependencies(const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData);
		
		void DiscoverAddedPackageDependencies(const FActivityNodeID NodeID, const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData);
		void DiscoverSavedPackageDependencies(const FActivityNodeID NodeID, const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData);
		void DiscoverRenamedPackageDependencies(const FActivityNodeID NodeID, const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData);
		void DiscoverDeletedPackageDependencies(const FActivityNodeID NodeID, const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData);

		void DiscoverPackageEditedDependencies(const FActivityNodeID DependingNodeID, FName ModifiedPackage);

		/** Util function for declarative style dependency statement in C++ */
		enum class EPackageAddDependencyCondition
		{
			/** The activity will be added unconditionally */
			Always,
			/** From the array of activities, only the one with the highest activity ID will be added. */
			OnlyLatestActivity
		};
		using FPackageActivityItem = TTuple<const FActivityID* /* ActivityID */, EActivityDependencyReason /* Reason */, EDependencyStrength /* DependencyStrength*/, EPackageAddDependencyCondition /* AddCondition */>;
		using FTrackedPackageActivityArray = TArray<FPackageActivityItem, TInlineAllocator<8>>;
		void AddDependencies(const FActivityNodeID NodeID, const FTrackedPackageActivityArray& Dependencies);
	
		void TrackAffectedPackages(const FConcertSyncActivity& Activity);
		void TrackAffectedPackages(const FConcertSyncActivity& Activity, const FConcertSyncTransactionEvent& EventData);
		void TrackAffectedPackages(const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData);

		FSoftObjectPath MakePathFromExportedObject(const FConcertExportedObject& ExportedObject) const;
	};
}

UE::ConcertSyncCore::FActivityDependencyGraph UE::ConcertSyncCore::BuildDependencyGraphFrom(const FConcertSyncSessionDatabase& SessionDatabase)
{
	return FActivityDependencyGraphBuildAlgorithm::BuildGraph(SessionDatabase);
}

namespace UE::ConcertSyncCore
{
	/**
	 * The algorithm works as follows:
	 *	For each transaction i sorted from earliest to later:
	 *		1. Add a dependency to every activity that has so far affected a package i depends on.
	 *		2. Track which packages i added, saved, renamed, removed, or modified.
	 *		
	 * Every transaction is processed exactly once, hence this algorithm is O(n).
	 */
	FActivityDependencyGraph FActivityDependencyGraphBuildAlgorithm::BuildGraph(const FConcertSyncSessionDatabase& SessionDatabase)
	{
		FActivityDependencyGraphBuildAlgorithm Builder(SessionDatabase);
		SessionDatabase.EnumerateActivities([&Builder](FConcertSyncActivity&& Activity)
		{
			if (Builder.IsRelevantForDependencies(Activity))
			{
				Builder.DiscoverPackageDependencies(Activity);
				Builder.TrackAffectedPackages(Activity);
			}

			return EBreakBehavior::Continue;
		});
		
		return Builder.Graph;
	}

	FActivityDependencyGraphBuildAlgorithm::FActivityDependencyGraphBuildAlgorithm(const FConcertSyncSessionDatabase& SessionDatabase)
		: SessionDatabase(SessionDatabase)
	{}

	bool FActivityDependencyGraphBuildAlgorithm::IsRelevantForDependencies(const FConcertSyncActivity& Activity)
	{
		return Activity.EventType == EConcertSyncActivityEventType::Package
			|| Activity.EventType == EConcertSyncActivityEventType::Transaction;
	}

	void FActivityDependencyGraphBuildAlgorithm::DiscoverPackageDependencies(const FConcertSyncActivity& Activity, const FConcertSyncTransactionEvent& EventData)
	{
		const FActivityID CurrentActivityID = Activity.ActivityId;
		const FActivityNodeID CurrentNodeID = Graph.AddActivity(CurrentActivityID);

		TSet<FActivityID> CreatedDependencies;
		for (const FConcertExportedObject& ExportedObject : EventData.Transaction.ExportedObjects)
		{
			const TPair<FActivityID, FPackageTracker::ESubobjectState>* LastActivity = PackageTracker.CreatedOrRemovedSubobjects.Find(MakePathFromExportedObject(ExportedObject));
			if (LastActivity)
			{
				const TOptional<FActivityNodeID> NodeID = Graph.FindNodeByActivity(LastActivity->Key);
				checkf(NodeID, TEXT("If the activity is in CreatedOrRemovedSubobjects, then we should have created a node for it in the graph!"));
				
				const EActivityDependencyReason Reason = LastActivity->Value == FPackageTracker::ESubobjectState::Created
					? EActivityDependencyReason::SubobjectCreation
					: EActivityDependencyReason::SubobjectRemoval;
				Graph.AddDependency(CurrentNodeID, FActivityDependencyEdge(*NodeID, Reason, EDependencyStrength::HardDependency));

				CreatedDependencies.Add(LastActivity->Key);
			}
		}

		auto AddDependency = [this, CurrentActivityID, CurrentNodeID](const FActivityID DependedOnActivityID, EActivityDependencyReason Reason, EDependencyStrength DependencyStrength)
		{
			const TOptional<FActivityNodeID> ActivityNodeID = Graph.FindNodeByActivity(DependedOnActivityID);
			if (ensureMsgf(ActivityNodeID, TEXT("Investigate algorithm. There should be a node for activity %lld because it supposed to have been processed before activity %lld. We assume that earlier activities have smaller IDs than later activities."), DependedOnActivityID, CurrentActivityID))
			{
				Graph.AddDependency(CurrentNodeID, FActivityDependencyEdge(*ActivityNodeID, Reason, DependencyStrength));
			}
		};
		
		for (const FName ModifiedPackage : EventData.Transaction.ModifiedPackages)
		{
			const FActivityID* LastModifyingActivity = PackageTracker.ModifiedPackages.Find(ModifiedPackage);
			const FActivityID* LastAddedActivity = PackageTracker.AddedPackages.Find(ModifiedPackage);
			const FActivityID* LastRenamingActivity = PackageTracker.RenamedToPackages.Find(ModifiedPackage);

			// We might depend on an activity that modified the same package but only if it is still the same package, i.e.
			// 1. The package was not added again between the previous transaction and the current (would imply delete or rename inbetween)
			// 2. The package was not renamed between the previous transaction and the current
			const bool bCanConsiderLastModifyingActivity = LastModifyingActivity
				&& (!LastAddedActivity || *LastAddedActivity < *LastModifyingActivity)
				&& (!LastRenamingActivity || *LastRenamingActivity < *LastModifyingActivity);
			if (bCanConsiderLastModifyingActivity)
			{
				// If there is already a hard dependency to a previous activity, then we do not "possibly" depend on it: we definitely depend on it (already).
				if (!CreatedDependencies.Contains(*LastModifyingActivity))
				{
					AddDependency(*LastModifyingActivity, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::PossibleDependency);
				}
			}
			// If nobody modified the package, we depend on the activity that added the package...
			else if (LastAddedActivity)
			{
				AddDependency(*LastAddedActivity, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency);
			}
			// ... or "created" it by renaming it
			else if (LastRenamingActivity)
			{
				AddDependency(*LastRenamingActivity, EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency);
			}

			/* We should consider indirect dependencies, too. Related ticket: UE-148392.
			 *
			 * Suppose:
			 *	1 Create data asset A
			 *	2 Make actor reference A
			 *	3 Edit data asset
			 *	4 Edit actor
			 *	
			 * In activity 4, the construction script may depend on the data asset modified in activity 3.
			 */
		}
	}

	void FActivityDependencyGraphBuildAlgorithm::DiscoverPackageDependencies(const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData)
	{
		const FActivityNodeID NodeID = Graph.AddActivity(
			Activity.ActivityId,
			EventData.PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Renamed
				? EActivityNodeFlags::RenameActivity
				: EActivityNodeFlags::None
			);
		switch (EventData.PackageInfo.PackageUpdateType)
		{
		case EConcertPackageUpdateType::Added:
			DiscoverAddedPackageDependencies(NodeID, Activity, EventData);
			break;
		case EConcertPackageUpdateType::Saved:
			DiscoverSavedPackageDependencies(NodeID, Activity, EventData);
			break;
		case EConcertPackageUpdateType::Renamed:
			DiscoverRenamedPackageDependencies(NodeID, Activity, EventData);
			break;
		case EConcertPackageUpdateType::Deleted:
			DiscoverDeletedPackageDependencies(NodeID, Activity, EventData);
			break;
			
		case EConcertPackageUpdateType::Dummy:
			// No idea what dependencies this has ... let's assume none and hopefully not regret it later
			break;
		default:
			checkNoEntry();
		}
	}

	void FActivityDependencyGraphBuildAlgorithm::DiscoverAddedPackageDependencies(const FActivityNodeID NodeID, const FConcertSyncActivity& Activity,const FConcertSyncPackageEventMetaData& EventData)
	{
		const FName& Package = EventData.PackageInfo.PackageName;
		const FActivityID* LastPackageRemoval = PackageTracker.RemovedPackages.Find(Package);
		const FActivityID* LastPackageRename = PackageTracker.RenamedToPackages.Find(Package);
		
		AddDependencies(
			NodeID,
			{
				{ LastPackageRemoval, EActivityDependencyReason::PackageRemoval, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity },
				{ LastPackageRename,  EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity }
			});
	}

	void FActivityDependencyGraphBuildAlgorithm::DiscoverSavedPackageDependencies(const FActivityNodeID NodeID, const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData)
	{
		const FName& Package = EventData.PackageInfo.PackageName;
		const FActivityID* LastPackageAddition = PackageTracker.AddedPackages.Find(Package);
		const FActivityID* LastPackageRename = PackageTracker.RenamedToPackages.Find(Package);

		const FTrackedPackageActivityArray Dependencies {
			{ LastPackageAddition, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity },
			{ LastPackageRename, EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity }
		};
		AddDependencies(NodeID, Dependencies);

		// Saving a package depends on the data that modified before it was saved
		DiscoverPackageEditedDependencies(NodeID, Package);
	}

	void FActivityDependencyGraphBuildAlgorithm::DiscoverRenamedPackageDependencies(const FActivityNodeID NodeID, const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData)
	{
		const FName& NewPackageName = EventData.PackageInfo.NewPackageName;
		const FName& OldPackageName = EventData.PackageInfo.PackageName;

		// 1. Rename depends on whatever created the new package
		const FActivityID* NewPackageAdded = PackageTracker.AddedPackages.Find(NewPackageName);
		const FActivityID* NewPackageRenamed = PackageTracker.RenamedToPackages.Find(NewPackageName);
		const FActivityID* LastActivityRenamingNewPackageName = PackageTracker.RenamedFromPackages.Find(NewPackageName);
		const FActivityID* SavedNewPackage = PackageTracker.SavedPackages.Find(NewPackageName);
		const FActivityID* DeleteNewPackage = PackageTracker.RemovedPackages.Find(NewPackageName);

		// An add package activity A is only a valid dependency if there was no delete between A and Activity.ActivityId. Similar for the other cases.
		const bool bIsNewPackageActivityValid = NewPackageAdded
			&& (!DeleteNewPackage || *DeleteNewPackage < *NewPackageAdded)
			&& (!LastActivityRenamingNewPackageName || *LastActivityRenamingNewPackageName < *NewPackageAdded);
		const bool bIsNewPackageRenamedValid = NewPackageRenamed
			&& (!NewPackageAdded || *NewPackageAdded < *NewPackageRenamed)
			&& (!LastActivityRenamingNewPackageName || *LastActivityRenamingNewPackageName < *NewPackageRenamed);
		
		FTrackedPackageActivityArray NewPackageDependencies;
		// When renaming, Concert does not generate a EConcertPackageUpdateType::Added type.
		// Instead the renamed to package is added via an EConcertPackageUpdateType::Saved activity which occurs just before
		// the corresponding EConcertPackageUpdateType::Rename activity.
		if (!bIsNewPackageActivityValid && !bIsNewPackageRenamedValid)
		{
			// Maybe this could be a hard dependency...
			NewPackageDependencies = {{ SavedNewPackage, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::Always }};
			
			// Saving a package depends on the data that modified before it was saved
			// SavedNewPackage was already processed but since the algorithm only looks backwards we didn't know the save operation was needed by a rename.
			// Now we know and must retrospectively update that nodes dependencies.
			if (SavedNewPackage)
			{
				const TOptional<FActivityNodeID> SaveNodeId = Graph.FindNodeByActivity(*SavedNewPackage);
				checkf(SaveNodeId.IsSet(), TEXT("SaveNodeId (%lld) preceeds the rename (%lld) so we should already have created a node for it in the graph!"), *SavedNewPackage, Activity.ActivityId);
				DiscoverPackageEditedDependencies(*SaveNodeId, OldPackageName);
			}
		}
		else
		{
			if (bIsNewPackageActivityValid)
			{
				NewPackageDependencies.Add({ NewPackageAdded, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity });
			}
			if (bIsNewPackageRenamedValid)
			{
				NewPackageDependencies.Add({ NewPackageRenamed, EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity });
			}
		}

		// 2. Rename depends on whatever created the old package
		const FActivityID* OldPackageAdded = PackageTracker.AddedPackages.Find(OldPackageName);
		const FActivityID* OldPackageRenamed = PackageTracker.RenamedToPackages.Find(OldPackageName);
		const FActivityID* LastActivityRenamingOldPackageName = PackageTracker.RenamedFromPackages.Find(OldPackageName);
		const FActivityID* SavedOldPackage = PackageTracker.SavedPackages.Find(OldPackageName);
		const FActivityID* DeleteOldPackage = PackageTracker.RemovedPackages.Find(OldPackageName);

		// See similar cases from above
		const bool bIsOldPackageActivityValid = OldPackageAdded
			&& (!DeleteOldPackage || *DeleteOldPackage < *OldPackageAdded)
			&& (!LastActivityRenamingOldPackageName || *LastActivityRenamingOldPackageName < *OldPackageAdded);
		const bool bIsOldPackageRenamedValid = OldPackageRenamed
			&& (!OldPackageAdded || *OldPackageAdded < *OldPackageRenamed)
			&& (!LastActivityRenamingOldPackageName || *LastActivityRenamingOldPackageName < *OldPackageRenamed);
		
		FTrackedPackageActivityArray OldPackageDependencies;
		// See above
		if (!bIsOldPackageActivityValid && !bIsOldPackageRenamedValid)
		{
			// Maybe this could be a hard dependency...
			OldPackageDependencies = {{ SavedOldPackage, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::Always }};
		}
		else
		{
			if (bIsOldPackageActivityValid)
			{
				OldPackageDependencies.Add({ OldPackageAdded, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity });
			}
			if (bIsOldPackageRenamedValid)
			{
				OldPackageDependencies.Add({ OldPackageRenamed, EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity });
			}
		}
		
		AddDependencies(NodeID, NewPackageDependencies);
		AddDependencies(NodeID, OldPackageDependencies);
	}

	void FActivityDependencyGraphBuildAlgorithm::DiscoverDeletedPackageDependencies(const FActivityNodeID NodeID, const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData)
	{
		const FName& Package = EventData.PackageInfo.PackageName;
		const FActivityID* LastPackageAddition = PackageTracker.AddedPackages.Find(Package);
		const FActivityID* LastPackageRename = PackageTracker.RenamedToPackages.Find(Package);
		
		AddDependencies(
			NodeID,
			{
				{ LastPackageAddition, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity},
				{ LastPackageRename, EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::OnlyLatestActivity }
			});
	}
	
	void FActivityDependencyGraphBuildAlgorithm::DiscoverPackageEditedDependencies(const FActivityNodeID DependingNodeID, FName ModifiedPackage)
	{
		TArray<FActivityID> TransactionsAffectingPackage;
		PackageTracker.TransactionsAffectingPackageSinceLastPackageActivity.RemoveAndCopyValue(ModifiedPackage, TransactionsAffectingPackage);
		for (const FActivityID& TransactionActivityID : TransactionsAffectingPackage)
		{
			AddDependencies(
				DependingNodeID, 
				{{ &TransactionActivityID, EActivityDependencyReason::PackageEdited, EDependencyStrength::HardDependency, EPackageAddDependencyCondition::Always }}
			);
		}
	}
	
	void FActivityDependencyGraphBuildAlgorithm::AddDependencies(const FActivityNodeID NodeID, const FTrackedPackageActivityArray& Dependencies)
	{
		auto AddDependency = [this, NodeID](const FPackageActivityItem& Dependency)
		{
			const FActivityID* ActivityID = Dependency.Get<0>();
			const TOptional<FActivityNodeID> TargetNodeID = Graph.FindNodeByActivity(*ActivityID);
			if (!ensureMsgf(TargetNodeID, TEXT("Investigate algorithm. There should be a node for activity %lld because it supposed to have been processed before activity %lld. We assume that earlier activities have smaller IDs than later activities."), *ActivityID, NodeID.ID))
			{
				return;
			}

			const EActivityDependencyReason Reason = Dependency.Get<1>();
			Graph.AddDependency(NodeID, FActivityDependencyEdge(*TargetNodeID, Reason, Dependency.Get<2>()));
		};
		
		const FPackageActivityItem* LatestDependency = nullptr;
		for (const FPackageActivityItem& Dependency : Dependencies)
		{
			const FActivityID* DependencyActivityID = Dependency.Get<0>();
			if (!DependencyActivityID)
			{
				continue;
			}
			
			if (Dependency.Get<3>() == EPackageAddDependencyCondition::Always)
			{
				AddDependency(Dependency);
			}
			else 
			{
				check(Dependency.Get<3>() == EPackageAddDependencyCondition::OnlyLatestActivity);
				const bool bCurrentActivityIsNewer = !LatestDependency || *LatestDependency->Get<0>() < *DependencyActivityID;
				if (bCurrentActivityIsNewer)
				{
					LatestDependency = &Dependency;
				}
			}
		}

		if (LatestDependency)
		{
			AddDependency(*LatestDependency);
		}
	}

	void FActivityDependencyGraphBuildAlgorithm::TrackAffectedPackages(const FConcertSyncActivity& Activity, const FConcertSyncTransactionEvent& EventData)
	{
		for (FName ModifiedPackage : EventData.Transaction.ModifiedPackages)
		{
			PackageTracker.ModifiedPackages.Add(ModifiedPackage, Activity.ActivityId);
			
			PackageTracker.TransactionsAffectingPackageSinceLastPackageActivity
				.FindOrAdd(ModifiedPackage)
				.Add(Activity.ActivityId);
		}
		
		for (const FConcertExportedObject& ExportedObject : EventData.Transaction.ExportedObjects)
		{
			if (ExportedObject.ObjectData.bAllowCreate && ensure(!ExportedObject.ObjectData.bIsPendingKill))
			{
				PackageTracker.CreatedOrRemovedSubobjects.Add(MakePathFromExportedObject(ExportedObject), { Activity.ActivityId, FPackageTracker::ESubobjectState::Created });
			}
			else if (ExportedObject.ObjectData.bIsPendingKill && ensure(!ExportedObject.ObjectData.bAllowCreate))
			{
				PackageTracker.CreatedOrRemovedSubobjects.Add(MakePathFromExportedObject(ExportedObject), { Activity.ActivityId, FPackageTracker::ESubobjectState::Removed });
			}
		}
	}

	void FActivityDependencyGraphBuildAlgorithm::TrackAffectedPackages(const FConcertSyncActivity& Activity, const FConcertSyncPackageEventMetaData& EventData)
	{
		const FName PackageName = EventData.PackageInfo.PackageName;
		const FActivityID ActivityID = Activity.ActivityId;
		switch (EventData.PackageInfo.PackageUpdateType)
		{
		case EConcertPackageUpdateType::Added:
			PackageTracker.AddedPackages.Add(PackageName, ActivityID);
			break;
			
		case EConcertPackageUpdateType::Saved:
			PackageTracker.SavedPackages.Add(PackageName, ActivityID);
			PackageTracker.TransactionsAffectingPackageSinceLastPackageActivity.Remove(PackageName);
			break;
			
		case EConcertPackageUpdateType::Renamed:
			PackageTracker.RenamedToPackages.Add(EventData.PackageInfo.NewPackageName, ActivityID);
			PackageTracker.RenamedFromPackages.Add(EventData.PackageInfo.PackageName, ActivityID);
			break;
			
		case EConcertPackageUpdateType::Deleted:
			PackageTracker.RemovedPackages.Add(PackageName, ActivityID);
			PackageTracker.TransactionsAffectingPackageSinceLastPackageActivity.Remove(PackageName);
			break;
			
		case EConcertPackageUpdateType::Dummy:
			// No idea what dependencies this has ... let's assume none and hopefully not regret it later
			break;
			
		default:
			checkNoEntry();
		}
	}
	
	FSoftObjectPath FActivityDependencyGraphBuildAlgorithm::MakePathFromExportedObject(const FConcertExportedObject& ExportedObject) const
	{
		return FSoftObjectPath(FString::Printf(TEXT("%s.%s"), *ExportedObject.ObjectId.ObjectOuterPathName.ToString(), *ExportedObject.ObjectId.ObjectName.ToString()));
	}

/** Checks whether Activity is a transaction or package activity, gets the event meta data, and forwards it to an overload of FuncName. */
#define FORWARD_ACTIVITY(Activity, FuncName) FConcertSyncTransactionEvent TransactionEvent; \
	FConcertSyncPackageEventMetaData PackageEventMetaData; \
	switch (Activity.EventType) \
	{ \
	case EConcertSyncActivityEventType::Transaction: \
		if (SessionDatabase.GetTransactionEvent(Activity.EventId, TransactionEvent)) \
		{ \
			FuncName(Activity, TransactionEvent); \
		} \
		else \
		{ \
			checkNoEntry(); \
		} \
		break; \
	case EConcertSyncActivityEventType::Package: \
		if (SessionDatabase.GetPackageEventMetaData(Activity.EventId, PackageEventMetaData.PackageRevision, PackageEventMetaData.PackageInfo)) \
		{ \
			FuncName(Activity, PackageEventMetaData); \
		} \
		else \
		{ \
			checkNoEntry(); \
		} \
		break; \
	case EConcertSyncActivityEventType::None: \
	case EConcertSyncActivityEventType::Connection: \
	case EConcertSyncActivityEventType::Lock: \
	default: \
		checkNoEntry(); \
	} \

	void FActivityDependencyGraphBuildAlgorithm::DiscoverPackageDependencies(const FConcertSyncActivity& Activity)
	{
		FORWARD_ACTIVITY(Activity, DiscoverPackageDependencies)
	}

	void FActivityDependencyGraphBuildAlgorithm::TrackAffectedPackages(const FConcertSyncActivity& Activity)
	{
		FORWARD_ACTIVITY(Activity, TrackAffectedPackages)
	}
}
