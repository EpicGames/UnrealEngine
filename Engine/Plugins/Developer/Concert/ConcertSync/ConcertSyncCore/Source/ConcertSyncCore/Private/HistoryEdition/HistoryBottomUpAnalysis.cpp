// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/HistoryAnalysis.h"

#include "HistoryEdition/ActivityDependencyEdge.h"

#include "Containers/Queue.h"
#include "HistoryEdition/ActivityDependencyGraph.h"
#include "HistoryEdition/ActivityNode.h"

namespace UE::ConcertSyncCore
{
	FHistoryAnalysisResult AnalyseActivityDependencies_BottomUp(const TSet<FActivityID>& ActivitiesToEdit, const FActivityDependencyGraph& DependencyGraph, bool bAddEditedAsHardDependencies)
	{
		FHistoryAnalysisResult Result;

		TSet<FActivityNodeID> PossibleDoubleEnqueuingProtection;
		TQueue<FActivityNodeID> PossibleDependencyActivitiesToAnalyse;
		FActivityNodeID CurrentActivityID;
		
		TSet<FActivityNodeID> HardDoubleEnqueuingProtection;
		TQueue<FActivityNodeID> HardDependencyActivitiesToAnalyse;
		for (const FActivityID ActivityToDelete : ActivitiesToEdit)
		{
			const TOptional<FActivityNodeID> NodeID = DependencyGraph.FindNodeByActivity(ActivityToDelete);
			if (ensureMsgf(NodeID, TEXT("Graph does not correspond to ActivitiesToDelete")))
			{
				HardDoubleEnqueuingProtection.Add(*NodeID);
				HardDependencyActivitiesToAnalyse.Enqueue(*NodeID);
			}
		}
		
		/* We check the hard dependencies first.
		 * Why? Example:
		 *
		 *		R
		 *	   / \
		 *	  A   B
		 *	   \ /
		 *	    C
		 *
		 * The edges C -> A -> R are possible dependencies.
		 * The edges C -> B -> R are hard dependencies.
		 *
		 * Now: delete R.
		 * We want C to be marked has a hard dependency.
		 */
		while (HardDependencyActivitiesToAnalyse.Dequeue(CurrentActivityID))
		{
			const FActivityNode& ActivityNode = DependencyGraph.GetNodeById(CurrentActivityID);

			const FActivityID ActivityID = ActivityNode.GetActivityId();
			if (bAddEditedAsHardDependencies || !ActivitiesToEdit.Contains(ActivityID))
			{
				Result.HardDependencies.Add(ActivityID);
			}
			
			for (const FActivityDependencyEdge& Dependency : ActivityNode.GetDependencies())
			{
				const FActivityNodeID& DependedOnNodeId = Dependency.GetDependedOnNodeID();
				if (Dependency.GetDependencyStrength() == EDependencyStrength::HardDependency)
				{
					if (!HardDoubleEnqueuingProtection.Contains(DependedOnNodeId))
					{
						HardDoubleEnqueuingProtection.Add(DependedOnNodeId);
						HardDependencyActivitiesToAnalyse.Enqueue(DependedOnNodeId);
					}
				}
				else
				{
					check(Dependency.GetDependencyStrength() == EDependencyStrength::PossibleDependency);

					if (!PossibleDoubleEnqueuingProtection.Contains(DependedOnNodeId))
					{
						PossibleDoubleEnqueuingProtection.Add(DependedOnNodeId);
						PossibleDependencyActivitiesToAnalyse.Enqueue(DependedOnNodeId);
					}
				}
			}
		}
		
		// Any possible dependencies that are not also hard dependencies can be added now
		while (PossibleDependencyActivitiesToAnalyse.Dequeue(CurrentActivityID))
		{
			// This would imply a hard dependency - hard dependency takes precedence over possible dependency
			if (HardDoubleEnqueuingProtection.Contains(CurrentActivityID))
			{
				continue;
			}
			
			const FActivityNode& ActivityNode = DependencyGraph.GetNodeById(CurrentActivityID);
			Result.PossibleDependencies.Add(ActivityNode.GetActivityId());
			
			for (const FActivityDependencyEdge& Dependency : ActivityNode.GetDependencies())
			{
				const FActivityNodeID& DependedOnNodeId = Dependency.GetDependedOnNodeID();
				if (Dependency.GetDependencyStrength() == EDependencyStrength::PossibleDependency
					&& !HardDoubleEnqueuingProtection.Contains(DependedOnNodeId)
					&& !PossibleDoubleEnqueuingProtection.Contains(DependedOnNodeId))
				{
					if (!PossibleDoubleEnqueuingProtection.Contains(DependedOnNodeId))
					{
						PossibleDoubleEnqueuingProtection.Add(DependedOnNodeId);
						PossibleDependencyActivitiesToAnalyse.Enqueue(DependedOnNodeId);
					}
				}
			}
		}
		
		return Result;
	}
}
