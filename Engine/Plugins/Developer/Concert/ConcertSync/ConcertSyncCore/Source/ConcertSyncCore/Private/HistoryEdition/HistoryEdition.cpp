// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/HistoryEdition.h"

#include "ConcertLogGlobal.h"
#include "ConcertSyncSessionDatabase.h"
#include "HistoryEdition/HistoryAnalysis.h"

namespace UE::ConcertSyncCore
{
	TSet<FActivityID> CombineRequirements(const FHistoryAnalysisResult& ToDelete)
	{
		TSet<FActivityID> Result;
		for (const FActivityID ActivityID : ToDelete.HardDependencies)
		{
			Result.Add(ActivityID);
		}
		for (const FActivityID ActivityID : ToDelete.PossibleDependencies)
		{
			Result.Add(ActivityID);
		}
		return Result;
	}
}

