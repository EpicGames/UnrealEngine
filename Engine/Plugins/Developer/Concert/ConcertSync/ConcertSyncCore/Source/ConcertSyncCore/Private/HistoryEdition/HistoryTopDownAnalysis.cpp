// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/HistoryAnalysis.h"

#include "Algo/Find.h"

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "HistoryEdition/ActivityDependencyGraph.h"
#include "HistoryEdition/DependencyGraphBuilder.h"

#include "Containers/Queue.h"
#include "HistoryEdition/ActivityNode.h"

namespace UE::ConcertSyncCore
{
	FHistoryAnalysisResult AnalyseActivityDependencies_TopDown(const TSet<FActivityID>& ActivitiesToDelete, const FConcertSyncSessionDatabase& Database, bool bAddActivitiesToDelete)
	{
		const FActivityDependencyGraph Graph = BuildDependencyGraphFrom(Database);
		return AnalyseActivityDependencies_TopDown(ActivitiesToDelete, Graph, bAddActivitiesToDelete);
	}

	/**
	 * If PossibleRenameActivityNode is a rename activity, then this function adds the activity that created the renamed-to package.
	 * This implies that PossibleRenameActivityNode is EConcertSyncActivityEventType::Package and has EConcertPackageUpdateType::Renamed.
	 */
	static void AddSavePackageActivityAssociatedWithRenamePackageActivity(const FActivityDependencyGraph& DependencyGraph, const FActivityNode& PossibleRenameActivityNode, TSet<FActivityNodeID>& DoubleEnqueuingProtection, TQueue<FActivityNodeID>& ActivitiesToAnalyse);
	
	FHistoryAnalysisResult AnalyseActivityDependencies_TopDown(const TSet<FActivityID>& ActivitiesToEdit, const FActivityDependencyGraph& DependencyGraph, bool bAddEditedAsHardDependencies)
	{
		FHistoryAnalysisResult Result;

		TSet<FActivityNodeID> PossibleDoubleEnqueuingProtection;
		TQueue<FActivityNodeID> PossibleDependencyActivitiesToAnalyse;
		FActivityNodeID CurrentActivityID;
		
		TSet<FActivityNodeID> HardDoubleEnqueuingProtection;
		TQueue<FActivityNodeID> HardDependencyActivitiesToAnalyse;
		for (const FActivityID ActivityToDelete : ActivitiesToEdit)
		{
			const TOptional<FActivityNodeID> NodeID = DependencyGraph.FindNodeByActivity(ActivityToDelete);
			if (ensureMsgf(NodeID, TEXT("Graph does not correspond to ActivitiesToDelete")))
			{
				HardDoubleEnqueuingProtection.Add(*NodeID);
				HardDependencyActivitiesToAnalyse.Enqueue(*NodeID);
				AddSavePackageActivityAssociatedWithRenamePackageActivity(DependencyGraph, DependencyGraph.GetNodeById(*NodeID), PossibleDoubleEnqueuingProtection, PossibleDependencyActivitiesToAnalyse);
			}
		}
		
		/* We check the hard dependencies first.
		 * Why? Example:
		 *
		 *		R
		 *	   / \
		 *	  A   B
		 *	   \ /
		 *	    C
		 *
		 * The edges C -> A -> R are possible dependencies.
		 * The edges C -> B -> R are hard dependencies.
		 *
		 * Now: delete R.
		 * We want C to be marked has a hard dependency.
		 */
		while (HardDependencyActivitiesToAnalyse.Dequeue(CurrentActivityID))
		{
			const FActivityNode& ActivityNode = DependencyGraph.GetNodeById(CurrentActivityID);

			const FActivityID ActivityID = ActivityNode.GetActivityId();
			if (bAddEditedAsHardDependencies || !ActivitiesToEdit.Contains(ActivityID))
			{
				Result.HardDependencies.Add(ActivityID);
			}
			
			for (const FActivityNodeID& ChildID : ActivityNode.GetAffectedChildren())
			{
				const FActivityNode& ChildNode = DependencyGraph.GetNodeById(ChildID);

				// Performance: The below iterates the edge list twice but usually there will 1 or 2 entries
				if (ChildNode.DependsOnActivity(ActivityID, DependencyGraph, {}, EDependencyStrength::HardDependency))
				{
					if (!HardDoubleEnqueuingProtection.Contains(ChildID))
					{
						HardDoubleEnqueuingProtection.Add(ChildID);
						HardDependencyActivitiesToAnalyse.Enqueue(ChildID);
						AddSavePackageActivityAssociatedWithRenamePackageActivity(DependencyGraph, ChildNode, PossibleDoubleEnqueuingProtection, PossibleDependencyActivitiesToAnalyse);
					}
				}
				else if (ChildNode.DependsOnActivity(ActivityID, DependencyGraph, {}, EDependencyStrength::PossibleDependency))
				{
					if (!HardDoubleEnqueuingProtection.Contains(ChildID) && !PossibleDoubleEnqueuingProtection.Contains(ChildID))
					{
						PossibleDoubleEnqueuingProtection.Add(ChildID);
						PossibleDependencyActivitiesToAnalyse.Enqueue(ChildID);
						AddSavePackageActivityAssociatedWithRenamePackageActivity(DependencyGraph, ChildNode, PossibleDoubleEnqueuingProtection, PossibleDependencyActivitiesToAnalyse);
					}
				}
			}
		}

		// Any possible dependencies that are not also hard dependencies can be added now
		while (PossibleDependencyActivitiesToAnalyse.Dequeue(CurrentActivityID))
		{
			// This would imply a hard dependency - hard dependency takes precedence over possible dependency
			if (HardDoubleEnqueuingProtection.Contains(CurrentActivityID))
			{
				continue;
			}
			
			const FActivityNode& ActivityNode = DependencyGraph.GetNodeById(CurrentActivityID);
			const FActivityID ActivityID = ActivityNode.GetActivityId();
			Result.PossibleDependencies.Add(ActivityNode.GetActivityId());
			for (const FActivityNodeID& ChildID : ActivityNode.GetAffectedChildren())
			{
				const FActivityNode& ChildNode = DependencyGraph.GetNodeById(ChildID);
				if (ChildNode.DependsOnActivity(ActivityID, DependencyGraph, {}, EDependencyStrength::PossibleDependency)
					&& !HardDoubleEnqueuingProtection.Contains(ChildID)
					&& !PossibleDoubleEnqueuingProtection.Contains(ChildID))
				{
					PossibleDoubleEnqueuingProtection.Add(ChildID);
					PossibleDependencyActivitiesToAnalyse.Enqueue(ChildID);
					AddSavePackageActivityAssociatedWithRenamePackageActivity(DependencyGraph, ChildNode, PossibleDoubleEnqueuingProtection, PossibleDependencyActivitiesToAnalyse);
				}
			}
		}

		return Result;
	}

	static void AddSavePackageActivityAssociatedWithRenamePackageActivity(const FActivityDependencyGraph& DependencyGraph, const FActivityNode& PossibleRenameActivityNode, TSet<FActivityNodeID>& DoubleEnqueuingProtection, TQueue<FActivityNodeID>& ActivitiesToAnalyse)
	{
		if ((PossibleRenameActivityNode.GetNodeFlags() & EActivityNodeFlags::RenameActivity) == EActivityNodeFlags::None)
		{
			return;
		}

		// Generally rename activities depends on at least one activity and at most on two:
			// 1. There is always a PossibleDependency with reason PackageCreation: this is the activity that created the renamed-to package; that's the one we want to add here)
			// 2. There is sometimes a HardDependency with reason PackageCreation in case the renamed package was created within the same session
		const FActivityDependencyEdge* PackageCreationDependency = Algo::FindByPredicate(PossibleRenameActivityNode.GetDependencies(),
			[](const FActivityDependencyEdge& Dependency)
			{
				const bool bIsAssociatedCreationActivity = Dependency.GetDependencyReason() == EActivityDependencyReason::PackageCreation && Dependency.GetDependencyStrength() == EDependencyStrength::HardDependency;
				return bIsAssociatedCreationActivity;
			});
		if (ensureMsgf(PackageCreationDependency, TEXT("Rename activities should have a hard dependency to the activity that creatd the renamed-to package"))
			&& !DoubleEnqueuingProtection.Contains(PackageCreationDependency->GetDependedOnNodeID()))
		{
			DoubleEnqueuingProtection.Add(PackageCreationDependency->GetDependedOnNodeID());
			ActivitiesToAnalyse.Enqueue(PackageCreationDependency->GetDependedOnNodeID());
		}
	}
}


