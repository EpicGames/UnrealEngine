// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ObjectPathHierarchy.h"

#include "Misc/ObjectPathUtils.h"

namespace UE::ConcertSyncCore
{
	namespace Private
	{
		template<typename T>
		static int32 IndexOfChildInUniquePtrArray(const TArray<TUniquePtr<T>>& Children, T* Child)
		{
			return Children.IndexOfByPredicate([Child](const TUniquePtr<T>& TreeNode)
			{
				return TreeNode.Get() == Child;
			});
		}
	}

	FObjectPathHierarchy::FObjectPathHierarchy(FObjectPathHierarchy&& Other)
		: AssetNodes(MoveTemp(Other.AssetNodes))
		, CachedNodes(MoveTemp(Other.CachedNodes))
	{}

	FObjectPathHierarchy& FObjectPathHierarchy::operator=(FObjectPathHierarchy&& Other)
	{
		if (this == &Other)
		{
			return *this;
		}
		
		AssetNodes = MoveTemp(Other.AssetNodes);
		CachedNodes = MoveTemp(Other.CachedNodes);
		return *this;
	}

	void FObjectPathHierarchy::TraverseTopToBottom(TFunctionRef<ETreeTraversalBehavior(const FChildRelation& Relation)> Callback, const FSoftObjectPath& Start) const
	{
		const bool bHasStart = !Start.IsNull();
		if (!bHasStart)
		{
			for (const TUniquePtr<FTreeNode>& Asset : AssetNodes)
			{
				if (!Asset->Children.IsEmpty()
					&& TraverseTopToBottomInternal(Asset->Data, Asset->Children, Callback) == ETreeTraversalBehavior::Break)
				{
					break;
				}
			}
		}
		else if (FTreeNode*const*const TreeNode = CachedNodes.Find(Start))
		{
			const FTreeNode& Node = **TreeNode;
			TraverseTopToBottomInternal(Node.Data, Node.Children, Callback);
		}
	}

	void FObjectPathHierarchy::TraverseBottomToTop(TFunctionRef<EBreakBehavior(const FChildRelation& Relation)> Callback, const FSoftObjectPath& Start) const
	{
		const bool bHasStart = !Start.IsNull();
		if (!bHasStart)
		{
			for (const TUniquePtr<FTreeNode>& Asset : AssetNodes)
			{
				if (!Asset->Children.IsEmpty()
					&& TraverseBottomToTopInternal(Asset->Data, Asset->Children, Callback) == EBreakBehavior::Break)
				{
					break;
				}
			}
		}
		else if (FTreeNode*const*const TreeNode = CachedNodes.Find(Start))
		{
			const FTreeNode& Node = **TreeNode;
			TraverseBottomToTopInternal(Node.Data, Node.Children, Callback);
		}
	}

	TOptional<EHierarchyObjectType> FObjectPathHierarchy::IsInHierarchy(const FSoftObjectPath& Object) const
	{
		FTreeNode*const*const TreeNode = CachedNodes.Find(Object);
		return TreeNode
			? (*TreeNode)->Data.Type
			: TOptional<EHierarchyObjectType>{};
	}

	bool FObjectPathHierarchy::HasChildren(const FSoftObjectPath& Object) const
	{
		FTreeNode*const*const TreeNode = CachedNodes.Find(Object);
		return TreeNode && !(*TreeNode)->Children.IsEmpty();
	}

	bool FObjectPathHierarchy::IsAssetInHierarchy(const FSoftObjectPath& Object) const
	{
		return AssetNodes.ContainsByPredicate([&Object](const TUniquePtr<FTreeNode>& Node)
		{
			return Node->Data.Object == Object;
		});
	}

	void FObjectPathHierarchy::AddObject(const FSoftObjectPath& ObjectPath)
	{
		if (FTreeNode** TreeNode = CachedNodes.Find(ObjectPath))
		{
			(*TreeNode)->Data.Type = EHierarchyObjectType::Explicit;
			return;
		}

		TUniquePtr<FTreeNode> NodeToInsert = MakeUnique<FTreeNode>(FObjectHierarchyInfo{ ObjectPath, EHierarchyObjectType::Explicit });
		// We'll walk up the hierarchy and in each iteration we insert a child node.
		for (FSoftObjectPath CurrentPath = ObjectPath; !CurrentPath.IsNull();)
		{
			CachedNodes.Add(CurrentPath, NodeToInsert.Get());
			
			// Base case: NodeToInsert is supposed to be a root node
			TOptional<FSoftObjectPath> OptionalParentPath = GetOuterPath(CurrentPath);
			if (!OptionalParentPath)
			{
				AssetNodes.Emplace(MoveTemp(NodeToInsert));
				return;
			}

			const FSoftObjectPath& ParentPath = *OptionalParentPath;
			FTreeNode** ParentNode = CachedNodes.Find(ParentPath);
			if (ParentNode)
			{
				// Just insert the node into the pre-existing hierarchy and we're done.
				NodeToInsert->Parent = *ParentNode;
				(*ParentNode)->Children.Emplace(MoveTemp(NodeToInsert));
				break;
			}
			
			// Parent does not exist, yet, so create it.
			TUniquePtr<FTreeNode> Parent = MakeUnique<FTreeNode>(FObjectHierarchyInfo{ ParentPath, EHierarchyObjectType::Implicit });
			NodeToInsert->Parent = Parent.Get();
			Parent->Children.Emplace(MoveTemp(NodeToInsert));

			// Advance the hierarchy:
			NodeToInsert = MoveTemp(Parent);
			CurrentPath = ParentPath;
		}
	}

	void FObjectPathHierarchy::RemoveObject(const FSoftObjectPath& Object)
	{
		FTreeNode** PossibleTreeNode = CachedNodes.Find(Object);
		if (!PossibleTreeNode)
		{
			return;
		}

		// This node is required by one of its children
		FTreeNode& NodeToRemove = **PossibleTreeNode;
		if (NodeToRemove.Data.Type == EHierarchyObjectType::Implicit)
		{
			ensureMsgf(!NodeToRemove.Children.IsEmpty(), TEXT("Broke invariant: there is supposed to be an explicitly added child."));
			return;
		}

		// If it has children, by invariant there must still be at least one child that is Explicit
		if (!NodeToRemove.Children.IsEmpty())
		{
			NodeToRemove.Data.Type = EHierarchyObjectType::Implicit;
			return;
		}

		// Walk up the hierarchy and destroy all nodes that were implicitly created for NodeToRemove
		bool bShouldStop = false;
		for (FTreeNode* CurrentNode = &NodeToRemove; CurrentNode && !bShouldStop;)
		{
			// Asset nodes do not have Parent. All other nodes have a valid Parent.
			if (CurrentNode->IsAssetNode()) 
			{
				// Order matters because CurrentNode will be destroyed when removed from AssetNodes.
				CachedNodes.Remove(CurrentNode->Data.Object);
				
				const int32 AssetIndex = Private::IndexOfChildInUniquePtrArray(AssetNodes, CurrentNode);
				check(AssetNodes.IsValidIndex(AssetIndex));
				AssetNodes.RemoveAtSwap(AssetIndex);
				return;
			}
			
			FTreeNode* Parent = CurrentNode->Parent;
			checkf(Parent, TEXT("CurrentNode %s is not an asset so it should have a parent"), *CurrentNode->Data.Object.ToString());
			{
				FTreeNode* NodePendingDestruction = CurrentNode;
				CachedNodes.Remove(NodePendingDestruction->Data.Object);
				
				const int32 ChildIndex = Private::IndexOfChildInUniquePtrArray(Parent->Children, NodePendingDestruction); 
				if (ensure(Parent->Children.IsValidIndex(ChildIndex)))
				{
					Parent->Children.RemoveAtSwap(ChildIndex); // ~TUniquePtr now destroys the node
				}
				
				// Memory pointed to by NodePendingDestruction has been destroyed - do not reference it anymore
				CurrentNode = Parent;
			}
			
			// No more walking up the chain if we encounter an explicitly added object or a node that another object generated implicitly
			bShouldStop = CurrentNode->Data.Type == EHierarchyObjectType::Explicit || CurrentNode->Children.Num() >= 1;
		}
	}

	ETreeTraversalBehavior FObjectPathHierarchy::TraverseTopToBottomInternal(
		const FObjectHierarchyInfo& OwnerData,
		const TArray<TUniquePtr<FTreeNode>>& Nodes,
		TFunctionRef<ETreeTraversalBehavior(const FChildRelation& Relation)> Callback
		)
	{
		for (const TUniquePtr<FTreeNode>& Node : Nodes)
		{
			const FObjectHierarchyInfo& ChildData = Node->Data;
			const FChildRelation Relation { OwnerData, ChildData };
			switch (Callback(Relation))
			{
			case ETreeTraversalBehavior::Continue:
				if (TraverseTopToBottomInternal(ChildData, Node->Children, Callback) == ETreeTraversalBehavior::Break)
				{
					return ETreeTraversalBehavior::Break;
				}
				continue;
			case ETreeTraversalBehavior::SkipSubtree: continue;
			case ETreeTraversalBehavior::Break: [[fallthrough]];
			default: return ETreeTraversalBehavior::Break;
			}
		}
		
		return ETreeTraversalBehavior::Continue;
	}

	EBreakBehavior FObjectPathHierarchy::TraverseBottomToTopInternal(
		const FObjectHierarchyInfo& OwnerData,
		const TArray<TUniquePtr<FTreeNode>>& Nodes,
		TFunctionRef<EBreakBehavior(const FChildRelation& Relation)> Callback
		)
	{
		for (const TUniquePtr<FTreeNode>& Node : Nodes)
		{
			if (TraverseBottomToTopInternal(Node->Data, Node->Children, Callback) == EBreakBehavior::Break)
			{
				return EBreakBehavior::Break;
			}
			
			const FObjectHierarchyInfo& ChildData = Node->Data;
			const FChildRelation Relation { OwnerData, ChildData };
			if (Callback(Relation) == EBreakBehavior::Break)
			{
				return EBreakBehavior::Break;
			}
		}
		
		return EBreakBehavior::Continue;
	}
}
