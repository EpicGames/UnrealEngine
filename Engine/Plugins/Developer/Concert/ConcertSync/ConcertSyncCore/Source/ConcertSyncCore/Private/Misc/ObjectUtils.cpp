// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ObjectUtils.h"

#include "Containers/UnrealString.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertSyncCore
{
	bool IsActor(const FSoftObjectPath& SoftObjectPath)
	{
		// Example of an actor called floor
		// SoftObjectPath = { AssetPath = {PackageName = "/Game/Maps/SyncBoxLevel", AssetName = "SyncBoxLevel"}, SubPathString = "PersistentLevel.Floor" } }
		const FUtf8String& SubPathString = SoftObjectPath.GetSubPathUtf8String();

		constexpr int32 PersistentLevelStringLength = 16; // "PersistentLevel." has 16 characters
		const bool bIsWorldObject = SubPathString.Contains("PersistentLevel.", ESearchCase::CaseSensitive);
		if (!bIsWorldObject)
		{
			// Not a path to a world object
			return {};
		}

		// Start search after the . behind PersistentLevel
		const int32 StartSearch = PersistentLevelStringLength + 1;
		const int32 IndexOfDotAfterActorName = SubPathString.Find(".", ESearchCase::CaseSensitive, ESearchDir::FromStart, StartSearch);
		return IndexOfDotAfterActorName == INDEX_NONE;
	}
	
	TOptional<FSoftObjectPath> GetActorOf(const FSoftObjectPath& SoftObjectPath)
	{
		// Example of an actor called floor
		// SoftObjectPath = { AssetPath = {PackageName = "/Game/Maps/SyncBoxLevel", AssetName = "SyncBoxLevel"}, SubPathString = "PersistentLevel.Floor" } }
		const FUtf8String& SubPathString = SoftObjectPath.GetSubPathUtf8String();

		constexpr int32 PersistentLevelStringLength = 16; // "PersistentLevel." has 16 characters
		const bool bIsWorldObject = SubPathString.Contains("PersistentLevel.", ESearchCase::CaseSensitive);
		if (!bIsWorldObject)
		{
			// Not a path to a world object
			return {};
		}

		// Start search after the . behind PersistentLevel
		const int32 StartSearch = PersistentLevelStringLength + 1;
		const int32 IndexOfDotAfterActorName = SubPathString.Find(".", ESearchCase::CaseSensitive, ESearchDir::FromStart, StartSearch);
		if (IndexOfDotAfterActorName == INDEX_NONE)
		{
			// SoftObjectPath points to an actor
			return {};
		}

		const int32 NumToChopOffRight = SubPathString.Len() - IndexOfDotAfterActorName;
		const FUtf8String NewSubstring = SubPathString.LeftChop(NumToChopOffRight);
		const FSoftObjectPath PathToOwningActor(SoftObjectPath.GetAssetPath(), NewSubstring);
		return PathToOwningActor;
	}
	
	FUtf8String ExtractObjectNameFromPath(const FSoftObjectPath& Object)
	{
		// Subpath looks like this PersistentLevel.Actor.Component
		const FUtf8String& Subpath = Object.GetSubPathUtf8String();
		const int32 LastDotIndex = Subpath.Find(".", ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (LastDotIndex == INDEX_NONE)
		{
			return {};
		}
		return Subpath.RightChop(LastDotIndex + 1);
	}

	TOptional<FSoftObjectPath> ReplaceActorInPath(const FSoftObjectPath& OldPath, const FSoftObjectPath& NewActor)
	{
		if (!IsActor(NewActor))
		{
			return {};
		}
		
		// Example of an actor called floor
		// SoftObjectPath = { AssetPath = {PackageName = "/Game/Maps/SyncBoxLevel", AssetName = "SyncBoxLevel"}, SubPathString = "PersistentLevel.Floor" } }
		const FUtf8String& OldSubPathString = OldPath.GetSubPathUtf8String();
		if (!OldSubPathString.Contains("PersistentLevel.", ESearchCase::CaseSensitive))
		{
			return {};
		}
		const FUtf8String& NewSubPathString = NewActor.GetSubPathUtf8String();
		
		constexpr int32 PersistentLevelStringLength = 16; // "PersistentLevel." has 16 characters
		constexpr int32 FirstActorCharIndex = PersistentLevelStringLength + 1;
		const int32 IndexOfDotAfterActorName_OldSubPath = OldSubPathString.Find(".", ESearchCase::CaseSensitive, ESearchDir::FromStart, FirstActorCharIndex);
		
		const bool bOldIsOnlyActor = !OldSubPathString.IsValidIndex(IndexOfDotAfterActorName_OldSubPath);
		if (bOldIsOnlyActor)
		{
			return NewActor;
		}

		const FUtf8String ReplacedSubPathString = NewSubPathString
			+ "."
			+ OldSubPathString.RightChop(IndexOfDotAfterActorName_OldSubPath + 1);
		return FSoftObjectPath(NewActor.GetAssetPath(), ReplacedSubPathString);
	}
};

