// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/AuthorityConflictSharedUtils.h"

#include "Replication/Data/ReplicationStream.h"
#include "Replication/Messages/ChangeAuthority.h"
#include "Replication/Messages/ChangeStream.h"
#include "Replication/Misc/IReplicationGroundTruth.h"

namespace UE::ConcertSyncCore::Replication::AuthorityConflictUtils
{
	namespace Private
	{
		static void ForEachClientWithPotentialConflict(
            const FSoftObjectPath& Object,
            TFunctionRef<EBreakBehavior(const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertySelection& ConflictingProperty)> Callback,
            TArrayView<const FGuid> IgnoredClients,
            const IReplicationGroundTruth& GroundTruth
            )
		{
			GroundTruth.ForEachClient([&Object, &Callback, &IgnoredClients, &GroundTruth](const FGuid& ClientEndpointId)
			{
				if (IgnoredClients.Contains(ClientEndpointId))
				{
					return EBreakBehavior::Continue;
				}
				
				EBreakBehavior Result = EBreakBehavior::Continue;
				GroundTruth.ForEachStream(ClientEndpointId, [&Object, &Callback, &ClientEndpointId, &GroundTruth, &Result](const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap) mutable
				{
					// If client has not claimed authority over this object in this stream, skip
					if (!GroundTruth.HasAuthority(ClientEndpointId, StreamId, Object))
					{
						return EBreakBehavior::Continue;
					}

					// This client is using the request object: report the potential conflict ...
					const FConcertObjectReplicationMap& ObjectReplicationMap = ReplicationMap;
					const FConcertReplicatedObjectInfo* ReplicationObjectInfo = ObjectReplicationMap.ReplicatedObjects.Find(Object);
					if (ReplicationObjectInfo && Callback(ClientEndpointId, StreamId, ReplicationObjectInfo->PropertySelection) == EBreakBehavior::Break)
					{
						// ... conflict ends iteration
						Result = EBreakBehavior::Break;
						return EBreakBehavior::Break;
					}
					// ... conflict resolved
					return EBreakBehavior::Continue;
				});
				return Result;
			});
		}
	}
	
	EAuthorityConflict EnumerateAuthorityConflicts(
		const FGuid& ClientId,
		const FSoftObjectPath& Object,
		const TSet<FConcertPropertyChain>& OverwriteProperties,
		const IReplicationGroundTruth& GroundTruth,
		FProcessAuthorityConflict ProcessConflict
		)
	{
		const FGuid IgnoredClients[] = { ClientId };

		bool bFreeOfConflicts = true;
		Private::ForEachClientWithPotentialConflict(
			Object,
			[&ProcessConflict, &OverwriteProperties, &bFreeOfConflicts](const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertySelection& WrittenProperties) mutable
			{
				EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
				const bool bHasOverlap = FConcertPropertySelection::EnumeratePropertyOverlaps(WrittenProperties.ReplicatedProperties, OverwriteProperties,
					[&ProcessConflict, &ClientId, &StreamId, &BreakBehavior](const FConcertPropertyChain& Overlap)
					{
						BreakBehavior = ProcessConflict(ClientId, StreamId, Overlap);
						return BreakBehavior;
					});
				
				// At this point we know that WritingClientId is sending WrittenProperties - if the properties overlap, the authority request is not possible
				const bool bHasNoConflict = !bHasOverlap;
				bFreeOfConflicts &= bHasNoConflict;
				return BreakBehavior;
			},
			IgnoredClients,
			GroundTruth
			);
		return bFreeOfConflicts ? EAuthorityConflict::Allowed : EAuthorityConflict::Conflict;
	}

	void CleanseConflictsFromAuthorityRequest(FConcertReplication_ChangeAuthority_Request& Request, const FGuid& SendingClient, const IReplicationGroundTruth& GroundTruth)
	{
		// Need to check whether TakeAuthority is taking authority over properties other clients are already replicating
		GroundTruth.ForEachStream(SendingClient, [&SendingClient, &Request, &GroundTruth](const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)
		{
			for (auto ChangeIt = Request.TakeAuthority.CreateIterator(); ChangeIt; ++ChangeIt)
			{
				TPair<FSoftObjectPath, FConcertStreamArray>& Change = *ChangeIt;
				
				const FConcertReplicatedObjectInfo* ObjectInfo = ReplicationMap.ReplicatedObjects.Find(Change.Key);
				if (!ObjectInfo)
				{
					ChangeIt.RemoveCurrent();
					continue;
				}

				const FSoftObjectPath& ObjectPath = Change.Key;
				const TSet<FConcertPropertyChain>& Properties = ObjectInfo->PropertySelection.ReplicatedProperties;
				const EAuthorityConflict Conflict = EnumerateAuthorityConflicts(SendingClient, ObjectPath, Properties, GroundTruth);

				const bool bHasConflict = Conflict == EAuthorityConflict::Conflict;
				FConcertStreamArray& StreamArray = Change.Value;
				if (bHasConflict && StreamArray.StreamIds.Num() > 1)
				{
					StreamArray.StreamIds.Remove(StreamId);
				}
				else if (bHasConflict)
				{
					ChangeIt.RemoveCurrent();
				}
			}

			return Request.TakeAuthority.IsEmpty() ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
	}
	
	void CleanseConflictsFromStreamRequest(FConcertReplication_ChangeStream_Request& Request, const FGuid& SendingClient, const IReplicationGroundTruth& GroundTruth)
	{
		// Need to check whether ObjectsToPut adds any properties that an existing client has authority over.
		GroundTruth.ForEachStream(SendingClient, [&SendingClient, &Request, &GroundTruth](const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)
		{
			for (auto ChangeIt = Request.ObjectsToPut.CreateIterator(); ChangeIt; ++ChangeIt)
			{
				const TPair<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& Change = *ChangeIt;
				const FConcertReplication_ChangeStream_PutObject& PutObject = Change.Value;

				// Just changing the class?
				if (PutObject.Properties.ReplicatedProperties.IsEmpty())
				{
					continue;
				}
				
				const FSoftObjectPath& ObjectPath = Change.Key.Object;
				const EAuthorityConflict Conflict = EnumerateAuthorityConflicts(SendingClient, ObjectPath, PutObject.Properties.ReplicatedProperties, GroundTruth);
				if (Conflict == EAuthorityConflict::Conflict)
				{
					ChangeIt.RemoveCurrent();
				}
			}

			const bool bIsRequestEmpty = Request.ObjectsToPut.IsEmpty();
			return bIsRequestEmpty ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
	}
}
