// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/ChangeStreamSharedUtils.h"

#include "Misc/EBreakBehavior.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/Messages/ChangeStream.h"

#include "Algo/NoneOf.h"

namespace UE::ConcertSyncCore::Replication::ChangeStreamUtils
{
	void ForEachRemovedObject(
		const FConcertReplication_ChangeStream_Request& Request,
		const TArray<FConcertReplicationStream>& ExistingStreams,
		TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID&)> Callback
		)
	{
		for (const FConcertReplicationStream& ExistingStream : ExistingStreams)
		{
			const FGuid& StreamId = ExistingStream.BaseDescription.Identifier;
			if (Request.StreamsToRemove.Contains(StreamId))
			{
				for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& RemovePair : ExistingStream.BaseDescription.ReplicationMap.ReplicatedObjects)
				{
					const FConcertObjectInStreamID RemovedObjectId {  StreamId, RemovePair.Key };
					if (Callback(RemovedObjectId) == EBreakBehavior::Break)
					{
						return;
					}
				}
			}
		}

		// This will call Callback multiple times if the same object is in ObjectsToRemove and StreamsToRemove
		// but that does not really matter (it is a weird case anyhow - why would anyone construct such a request?)
		for (const FConcertObjectInStreamID& ObjectToRemove : Request.ObjectsToRemove)
		{
			if (Callback(ObjectToRemove) == EBreakBehavior::Break)
			{
				return;
			}
		}
	}

	void ForEachAddedObject(
		const FConcertReplication_ChangeStream_Request& Request,
		const TArray<FConcertReplicationStream>& ExistingStreams,
		TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID&)> Callback
		)
	{
		for (const TPair<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& PutObject : Request.ObjectsToPut)
		{
			const bool bIsNewObject = Algo::NoneOf(ExistingStreams, [&PutObject](const FConcertReplicationStream& Stream)
			{
				return Stream.BaseDescription.ReplicationMap.HasProperties(PutObject.Key.Object);
			});
			if (bIsNewObject && Callback(PutObject.Key) == EBreakBehavior::Break)
			{
				return;
			}
		}
		
		for (const FConcertReplicationStream& AddedStream : Request.StreamsToAdd)
		{
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ObjectInfo : AddedStream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				if (Callback({ AddedStream.BaseDescription.Identifier, ObjectInfo.Key }) == EBreakBehavior::Break)
				{
					return;
				}
			}
		}
	}
	
	void ApplyValidatedRequest(const FConcertReplication_ChangeStream_Request& Request, TArray<FConcertReplicationStream>& StreamsToModify)
	{
		for (auto StreamIt = StreamsToModify.CreateIterator(); StreamIt; ++StreamIt)
		{
			FConcertBaseStreamInfo& BaseDescription = StreamIt->BaseDescription;
			TMap<FSoftObjectPath, FConcertReplicatedObjectInfo>& ReplicationMap = BaseDescription.ReplicationMap.ReplicatedObjects;
			FConcertStreamFrequencySettings& FrequencySettings = BaseDescription.FrequencySettings;
			const FGuid& StreamId = BaseDescription.Identifier;
			
			for (const FConcertObjectInStreamID& ObjectToRemove : Request.ObjectsToRemove)
			{
				if (ObjectToRemove.StreamId == StreamId)
				{
					ReplicationMap.Remove(ObjectToRemove.Object);
					FrequencySettings.ObjectOverrides.Remove(ObjectToRemove.Object);
				}
			}

			if (const FConcertReplication_ChangeStream_Frequency* FrequencyChange = Request.FrequencyChanges.Find(StreamId))
			{
				ApplyValidatedFrequencyChanges(*FrequencyChange, FrequencySettings);
			}
			
			if (Request.StreamsToRemove.Contains(StreamId)
				// By contract, ObjectsToRemove will remove the stream if the stream ends up being empty
				|| ReplicationMap.IsEmpty())
			{
				StreamIt.RemoveCurrent();
			}
			
			for (const TPair<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& PutObjectPair : Request.ObjectsToPut)
			{
				if (PutObjectPair.Key.StreamId != StreamId)
				{
					continue;
				}

				const FSoftClassPath& PathToSet = PutObjectPair.Value.ClassPath;
				const FConcertPropertySelection& SelectionToSet = PutObjectPair.Value.Properties;
				FConcertReplicatedObjectInfo& ObjectInfo = ReplicationMap.FindOrAdd(PutObjectPair.Key.Object);

				// Write ClassPath if FindOrAdd just added it or if request specified a value to overwrite with ...
				if (!ObjectInfo.ClassPath.IsValid() || PathToSet.IsValid())
				{
					ObjectInfo.ClassPath = PutObjectPair.Value.ClassPath;
				}
				// ... but one of these cases must occur
				checkf(ObjectInfo.ClassPath.IsValid(), TEXT("Request did not validate ClassPath!"));

				// Write Properties if FindOrAdd just added it or if request specified a value to overwrite with ...
				if (ObjectInfo.PropertySelection.ReplicatedProperties.IsEmpty() || !SelectionToSet.ReplicatedProperties.IsEmpty())
				{
					ObjectInfo.PropertySelection = SelectionToSet;
				}
				// ... but one of these cases must occur
				checkf(!ObjectInfo.PropertySelection.ReplicatedProperties.IsEmpty(), TEXT("Request did not validate Properties!"));
			}
		}

		for (const FConcertReplicationStream& StreamDescription : Request.StreamsToAdd)
		{
			ensureAlwaysMsgf(!StreamDescription.BaseDescription.ReplicationMap.IsEmpty(), TEXT("This should have been validated"));
			StreamsToModify.Add(StreamDescription);
		}
	}
	
	void ApplyValidatedFrequencyChanges(
		const FConcertReplication_ChangeStream_Frequency& Request,
		FConcertStreamFrequencySettings& SettingsToModify
		)
	{
		if (EnumHasAnyFlags(Request.Flags, EConcertReplicationChangeFrequencyFlags::SetDefaults)
			&& ensure(Request.NewDefaults.IsValid()))
		{
			SettingsToModify.Defaults = Request.NewDefaults;
		}

		if (!Request.OverridesToPut.IsEmpty())
		{
			SettingsToModify.ObjectOverrides = Request.OverridesToPut;
		}

		for (const FSoftObjectPath& ToRemove : Request.OverridesToRemove)
		{
			SettingsToModify.ObjectOverrides.Remove(ToRemove);
		}
		
		for (const TPair<FSoftObjectPath, FConcertObjectReplicationSettings>& ToAdd : Request.OverridesToAdd)
		{
			if (ensure(ToAdd.Value.IsValid()))
			{
				SettingsToModify.ObjectOverrides.Add(ToAdd.Key, ToAdd.Value);
			}
		}
	}

#define ADD_ERROR(Condition, Callback) if (Condition) { Callback; }
	namespace Private
	{
		static void MarkAsMissingStream(const FGuid& StreamId, const TMap<FSoftObjectPath, FConcertObjectReplicationSettings>& Added, FConcertReplication_ChangeStream_FrequencyResponse& Errors)
		{
			for (const TPair<FSoftObjectPath, FConcertObjectReplicationSettings>& Pair : Added)
			{
				Errors.OverrideFailures.Add({ StreamId, Pair.Key }, EConcertChangeObjectFrequencyErrorCode::NotRegistered);
			}
		}

		static bool HasOrIsAddingProperties(const FConcertReplication_ChangeStream_Request& Request, const FConcertBaseStreamInfo& Stream, const FSoftObjectPath& ObjectPath)
		{
			const FConcertObjectReplicationMap& ReplicationMap = Stream.ReplicationMap;
			return ReplicationMap.HasProperties(ObjectPath)
				// It might be that ObjectsToPut is invalid.
				// We do not need to check that here because the underlying FConcertReplication_ChangeStream_Request will be checked separately and would fail then anyways.
				|| Request.ObjectsToPut.Contains({ Stream.Identifier, ObjectPath });
		}
		
		static bool ValidateAddedFrequencies(
			const FConcertReplication_ChangeStream_Request& FullRequest,
			const TMap<FSoftObjectPath, FConcertObjectReplicationSettings>& Added,
			const FConcertBaseStreamInfo& Stream,
			FConcertReplication_ChangeStream_FrequencyResponse* OptionalErrors = nullptr)
		{
			bool bAnyError = false;
			for (const TPair<FSoftObjectPath, FConcertObjectReplicationSettings>& Pair : Added)
			{
				if (!HasOrIsAddingProperties(FullRequest, Stream, Pair.Key))
				{
					bAnyError = true;
					ADD_ERROR(OptionalErrors, OptionalErrors->OverrideFailures.Add({ Stream.Identifier, Pair.Key }, EConcertChangeObjectFrequencyErrorCode::NotRegistered));
				}
				else if (!Pair.Value.IsValid())
				{
					ADD_ERROR(OptionalErrors, OptionalErrors->OverrideFailures.Add({ Stream.Identifier, Pair.Key }, EConcertChangeObjectFrequencyErrorCode::InvalidReplicationRate));
				}
			}
			
			return bAnyError;
		}
	}

	bool ValidateFrequencyChanges(
		const FConcertReplication_ChangeStream_Request& Request,
		const TConstArrayView<FConcertReplicationStream>& Streams,
		FConcertReplication_ChangeStream_FrequencyResponse* OptionalErrors
		)
	{
		using namespace Private;
		bool bAnyErrors = false;
		
		for (const TPair<FGuid, FConcertReplication_ChangeStream_Frequency>& SubRequestPair : Request.FrequencyChanges)
		{
			const FGuid& TargetStreamId = SubRequestPair.Key;
			const FConcertReplication_ChangeStream_Frequency& SubRequest = SubRequestPair.Value;
			
			const bool bIsModifyingDefaults = EnumHasAnyFlags(SubRequest.Flags, EConcertReplicationChangeFrequencyFlags::SetDefaults);
			const FConcertReplicationStream* StreamDescription = Streams.FindByPredicate([&TargetStreamId](const FConcertReplicationStream& Description)
			{
				return Description.BaseDescription.Identifier == TargetStreamId;
			});

			// Referenced stream does not exist
			if (!StreamDescription)
			{
				// TODO UE-201167: Once StreamsToAdd no longer specifies objects directly, we need to handle that case here
				
				bAnyErrors = true;
				ADD_ERROR(OptionalErrors && bIsModifyingDefaults, OptionalErrors->DefaultFailures.Add(TargetStreamId, EConcertChangeStreamFrequencyErrorCode::UnknownStream))
				ADD_ERROR(OptionalErrors, MarkAsMissingStream(TargetStreamId, SubRequest.OverridesToPut, *OptionalErrors))
				ADD_ERROR(OptionalErrors, MarkAsMissingStream(TargetStreamId, SubRequest.OverridesToAdd, *OptionalErrors))
				continue;
			}

			// Defaults must be valid
			if (bIsModifyingDefaults && !SubRequest.NewDefaults.IsValid())
			{
				bAnyErrors = true;
				ADD_ERROR(OptionalErrors && bIsModifyingDefaults, OptionalErrors->DefaultFailures.Add(TargetStreamId, EConcertChangeStreamFrequencyErrorCode::InvalidReplicationRate))
			}

			// Object paths referenced must have properties mapped to them in the replication map
			bAnyErrors |= ValidateAddedFrequencies(Request, SubRequest.OverridesToPut, StreamDescription->BaseDescription, OptionalErrors);
			bAnyErrors |= ValidateAddedFrequencies(Request, SubRequest.OverridesToAdd, StreamDescription->BaseDescription, OptionalErrors);
		}
		
		return bAnyErrors;
	}
#undef ADD_ERROR
	
	void IterateInvalidEntries(
		const FConcertObjectReplicationMap& ReplicationMap,
		TFunctionRef<EBreakBehavior(const FSoftObjectPath&, const FConcertReplicatedObjectInfo&)> Callback
		)
	{
		for (const TPair<const FSoftObjectPath&, const FConcertReplicatedObjectInfo&> Pair : ReplicationMap.ReplicatedObjects)
		{
			if (!Pair.Value.IsValidForSendingToServer() && Callback(Pair.Key, Pair.Value) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	namespace Private
	{
		static void BuildPutObjectList(const FGuid& StreamId, const FConcertObjectReplicationMap& Base, const FConcertObjectReplicationMap& Desired, FConcertReplication_ChangeStream_Request& Request)
		{
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& BasePair : Base.ReplicatedObjects)
			{
				const FSoftObjectPath& ObjectPath = BasePair.Key;
				const FConcertObjectInStreamID ObjectId { StreamId, ObjectPath };
				const FConcertReplicatedObjectInfo* DesiredObjectInfo = Desired.ReplicatedObjects.Find(ObjectPath);
				if (DesiredObjectInfo)
				{
					const FConcertReplicatedObjectInfo& BaseObjectInfo = BasePair.Value;
					const TOptional<FConcertReplication_ChangeStream_PutObject> PutObject = FConcertReplication_ChangeStream_PutObject::MakeFromChange(BaseObjectInfo, *DesiredObjectInfo);
				
					const bool bDesiredHasChangedFromBase = BaseObjectInfo != *DesiredObjectInfo;
					// If MakeFromChange returned unset, it means that this request is not valid to submit. 
					const bool bBaseAndDesiredStateAreValid = ensureMsgf(PutObject, TEXT("Function assumption violated; you did not pass in valid base or desired state."));
					if (bDesiredHasChangedFromBase && bBaseAndDesiredStateAreValid)
					{
						Request.ObjectsToPut.Add(ObjectId, *PutObject);
					}
				}
				else
				{
					Request.ObjectsToRemove.Add(ObjectId);
				}
			}
		}

		static void BuildRemoveObjectList(const FGuid& StreamId, const FConcertObjectReplicationMap& Base, const FConcertObjectReplicationMap& Desired, FConcertReplication_ChangeStream_Request& Request)
		{
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& DesiredPair : Desired.ReplicatedObjects)
			{
				const FSoftObjectPath& ObjectPath = DesiredPair.Key;
				if (Base.ReplicatedObjects.Contains(ObjectPath))
				{
					// Handled up above
					continue;
				}

				// Desired wants to add an object
				const FConcertReplicatedObjectInfo& ObjectInfo = DesiredPair.Value;
				const TOptional<FConcertReplication_ChangeStream_PutObject> PutObject = FConcertReplication_ChangeStream_PutObject::MakeFromInfo(ObjectInfo);
				const bool bDesiredStateHasEnoughDataToForPut = ensureMsgf(PutObject, TEXT("Function assumption violated; you did not pass in valid base or desired state."));
				if (bDesiredStateHasEnoughDataToForPut)
				{
					Request.ObjectsToPut.Add({ StreamId, ObjectPath}, *PutObject);
				}
			}
		}
	}

	FConcertReplication_ChangeStream_Request BuildRequestFromDiff(
		const FGuid& StreamId,
		const FConcertObjectReplicationMap& Base,
		const FConcertObjectReplicationMap& Desired
		)
	{
		FConcertReplication_ChangeStream_Request Request;
		Private::BuildPutObjectList(StreamId, Base, Desired, Request);
		Private::BuildRemoveObjectList(StreamId, Base, Desired, Request);
		return Request;
	}
}
