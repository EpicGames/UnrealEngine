// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Data/ConcertPropertySelection.h"

#include "ConcertLogGlobal.h"
#include "Replication/PropertyChainUtils.h"

#include "Algo/AllOf.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertPropertySelection)

TOptional<FConcertPropertyChain> FConcertPropertyChain::CreateFromPath(const UStruct& Class, const TArray<FName>& NamePath)
{
    TOptional<FConcertPropertyChain> Result;
    // The goal here is to find a valid FProperty path based on the NamePath...
    UE::ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(Class, [&NamePath, &Result](FConcertPropertyChain&& Chain) mutable
    {
       // ... and performance could be improved by not expanding down the search tree by introducing ETreeBreakBehavior::SkipSubtree
       if (Chain == NamePath)
       {
          Result.Emplace(MoveTemp(Chain));
          return EBreakBehavior::Break;
       }
       return EBreakBehavior::Continue;
    });
    return Result;
}

FConcertPropertyChain::FConcertPropertyChain(const FArchiveSerializedPropertyChain* OptionalChain, const FProperty& LeafProperty)
{
    using namespace UE::ConcertSyncCore::PropertyChain;
    
    if (!ensureMsgf(!IsInnerContainerProperty(LeafProperty), TEXT("Concert property chains never contain inner properties so you cannot construct one to point at it!")))
    {
       return;
    }
    
    if (OptionalChain)
    {
       PathToProperty.Reserve(OptionalChain->GetNumProperties() + 1);
       for (int32 i = 0; i < OptionalChain->GetNumProperties(); ++i)
       {
          const FProperty* CurrentProperty = OptionalChain->GetPropertyFromRoot(i);
          
          const FProperty* ParentProperty = CurrentProperty->GetOwner<FProperty>();
          const bool bIsKeyOfMap = ParentProperty && ParentProperty->IsA(FMapProperty::StaticClass()) && CastField<FMapProperty>(ParentProperty)->KeyProp == CurrentProperty;
          if (!ensureMsgf(!bIsKeyOfMap, TEXT("Key child properties never appear in a path because all of them are assumed to be replicated. See documentation of PathToProperty!")))
          {
             // The input was invalid but we've constructed FConcertPropertyChain with valid state: it ends at the TMap property.
             return;
          }

          // Inner properties do not show up in the path but are expected to be contained in OptionalChain since it's needed for the search.
          if (!IsInnerContainerProperty(*CurrentProperty))
          {
             PathToProperty.Add(OptionalChain->GetPropertyFromRoot(i)->GetFName());
          }
       }
    }
    else if (!ensureMsgf(LeafProperty.GetOwner<FProperty>() == nullptr, TEXT("Not a leaf property!")))
    {
       // The input was invalid but we've constructed FConcertPropertyChain with valid state: no property.
       return;
    }
    
    PathToProperty.Add(LeafProperty.GetFName());
    UE_CLOGF(!IsReplicatableProperty(LeafProperty), LogConcert, Warning, "Instantiated property chain with non replicatable property: %ls", *ToString());
}

bool FConcertPropertyChain::IsChildOf(const FConcertPropertyChain& ParentToCheck) const
{
    if (PathToProperty.Num() < ParentToCheck.PathToProperty.Num())
    {
       return false;
    }

    for (int32 i = 0; i < ParentToCheck.PathToProperty.Num(); ++i)
    {
       if (PathToProperty[i] != ParentToCheck.PathToProperty[i])
       {
          return false;
       }
    }

    return true;
}

bool FConcertPropertyChain::IsDirectChildOf(const FConcertPropertyChain& ParentToCheck) const
{
    if (ParentToCheck.PathToProperty.Num() != PathToProperty.Num() - 1)
    {
       return false;
    }

    bool bArePathsEqual = true;
    for (int32 i = 0; bArePathsEqual && i < ParentToCheck.PathToProperty.Num(); ++i)
    {
       bArePathsEqual &= ParentToCheck.PathToProperty[i] == PathToProperty[i];
    }
    return bArePathsEqual;
}

bool FConcertPropertyChain::MatchesExactly(const FArchiveSerializedPropertyChain* OptionalChain, const FProperty& LeafProperty) const
{
    using namespace UE::ConcertSyncCore::PropertyChain;
    if (PathToProperty.IsEmpty())
    {
       return false;
    }
    
    const int32 OptionalChainLength = OptionalChain ? OptionalChain->GetNumProperties() : 0;
    const bool bLeafPropertiesMatch = LeafProperty.GetFName() == PathToProperty[PathToProperty.Num() - 1];
    // No point continuing matching properties if leaf properties do not match.
    if (!bLeafPropertiesMatch)
    {
       return false;
    }

    // Does the chain describe a single property? If yes, we're done.
    if (OptionalChainLength == 0 && PathToProperty.Num() == 1)
    {
       return true;
    }

    // We get here if PathToproperty.Num() > 1 so OptionalChain must be valid, otherwise we're not equal.
    if (!OptionalChain)
    {
       return false;
    }

    /*
     * OptionalChainLength can be larger than PathToProperty.Num() - 1 and still match!
     * 
     * Example:
     * USTRUCT() struct FInner { UPROPERTY() float Value; }
     * UCLASS() class UFoo : public UObject { UPROPERTY() TArray<FInner> Nested; }
     * 
     * Supposing PathToProperty = { "Nested", "Value" }, this is what the input to Matches would be:
     *  - OptionalChain = { "Nested", "Nested" } ==> 1st Nested is the FArrayProperty, 2nd the FArrayProperty::Inner
     *  - LeafProperty = "Value"
     */    
    bool bArePathsEqual = true;

    int32 Index_PathToProperty = 0;
    int32 Index_OptionalChain = 0;
    // Advance Index_OptionalChain towards the end of OptionalChain...
    for (; bArePathsEqual && PathToProperty.IsValidIndex(Index_PathToProperty) && Index_OptionalChain < OptionalChain->GetNumProperties(); ++Index_OptionalChain)
    {
       const FProperty* OptionalChainProperty = OptionalChain->GetPropertyFromRoot(Index_OptionalChain);
       const bool bIsInternalContainerProperty = IsInnerContainerProperty(*OptionalChainProperty);
       bArePathsEqual &= bIsInternalContainerProperty || OptionalChainProperty->GetFName() == PathToProperty[Index_PathToProperty];

       // ... but only advance Index_PathToProperty if the current property is not an inner container property since they do not show up in FConcertPropertyChains.
       if (!bIsInternalContainerProperty)
       {
          ++Index_PathToProperty;
       }
    }

    // If not every property was visited, that could mean one path is is a sub-path of the other. Regardless of the reason, they stopped matching somewhere in the middle so return false.
    const bool bVisitedEveryProperty = Index_OptionalChain == OptionalChain->GetNumProperties()
       // -1 to exclude leaf property (we checked it way at the beginning): OptionalChain does not contain the leaf property.
       && (Index_PathToProperty == PathToProperty.Num() - 1);
    return bArePathsEqual && bVisitedEveryProperty;
}

FProperty* FConcertPropertyChain::ResolveProperty(const UStruct& Class, bool bLogOnFail) const
{
    // FConcertPropertyChain::ResolveProperty exists only for visibility to developers since
    // the class is the first place one would look and not in the utils namespace.
    return UE::ConcertSyncCore::PropertyChain::ResolveProperty(Class, *this, bLogOnFail);
}

FConcertPropertyChain FConcertPropertyChain::GetParent() const
{
    if (PathToProperty.Num() <= 1)
    {
       return {};
    }
    
    FConcertPropertyChain Result = *this;
    Result.PathToProperty.RemoveAt(Result.PathToProperty.Num() - 1);
    return Result;
}

FConcertPropertyChain FConcertPropertyChain::GetRootParent() const
{
	if (PathToProperty.Num() <= 1)
	{
		return {};
	}
	
	FConcertPropertyChain Result = *this;
	Result.PathToProperty = { PathToProperty[0] };
	return Result;
}

FString FConcertPropertyChain::ToString(EToStringMethod Method) const
{
    switch (Method)
    {
    case EToStringMethod::Path:
       return FString::JoinBy(PathToProperty, TEXT("."), [](const FName& Name){ return Name.ToString(); });
    case EToStringMethod::LeafProperty:
       return PathToProperty.IsEmpty()
          ? FName(NAME_None).ToString()
          : PathToProperty[PathToProperty.Num() - 1].ToString();
    default:
       checkNoEntry();
       return FName(NAME_None).ToString();
    }
}

void FConcertPropertySelection::DiscoverAndAddImplicitParentProperties()
{
    // Iterate in reverse to ignore the properties we're about to add
    for (FConcertPropertyChain Current : TSet(ReplicatedProperties))
    {
       while (!Current.IsEmpty())
       {
          ReplicatedProperties.Add(Current);
          Current = Current.GetParent();
       }
    }
}

bool FConcertPropertySelection::EnumeratePropertyOverlaps(
    const TSet<FConcertPropertyChain>& First,
    const TSet<FConcertPropertyChain>& Second,
    TFunctionRef<EBreakBehavior(const FConcertPropertyChain&)> Callback
    )
{
    const TSet<FConcertPropertyChain> Intersection = First.Intersect(Second);
    for (const FConcertPropertyChain& ThisChain : Intersection)
    {
       if (Callback(ThisChain) == EBreakBehavior::Break) 
       {
          return true;
       }
    }

    return !Intersection.IsEmpty();
}

uint32 UE::ConcertSyncCore::ComputeHashForPropertyChainContent(const TArray<FName>& PropertyChain)
{
    return GetTypeHash(PropertyChain);
}
