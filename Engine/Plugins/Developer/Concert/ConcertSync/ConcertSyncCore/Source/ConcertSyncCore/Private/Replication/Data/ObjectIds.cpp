// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Data/ObjectIds.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectIds)

uint32 GetTypeHash(const FConcertObjectInStreamID& StreamObject)
{
	return HashCombineFast(GetTypeHash(StreamObject.Object), GetTypeHash(StreamObject.StreamId));
}

uint32 GetTypeHash(const FConcertReplicatedObjectId& StreamObject)
{
	return HashCombineFast(GetTypeHash(StreamObject.SenderEndpointId), GetTypeHash(static_cast<FConcertObjectInStreamID>(StreamObject)));
}
