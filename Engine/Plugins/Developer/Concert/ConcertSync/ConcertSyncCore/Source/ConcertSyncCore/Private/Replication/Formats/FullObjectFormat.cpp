// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Formats/FullObjectFormat.h"

#include "ConcertMessageData.h"
#include "Replication/Formats/FullObjectReplicationData.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"

namespace UE::ConcertSyncCore
{
	namespace FullObjectFormat
	{
		static void ApplySharedArchiveConfig(FArchive& Archive, bool bIsLoading)
		{
			Archive.ArIgnoreArchetypeRef = true;
			Archive.ArIgnoreClassRef = true;
			Archive.ArNoDelta = true; // Always serialize CDO values
			Archive.SetUseUnversionedPropertySerialization(false); // This is faster - Concert ensures all clients are on the same version
			// We'd like to use binary serialization but sadly we must used tagged serialization since otherwise the properties cannot be mapped
			//Archive.SetWantBinaryPropertySerialization(true);
			
			Archive.SetIsSaving(!bIsLoading);
			Archive.SetIsLoading(bIsLoading);
			Archive.SetIsPersistent(false);
		}
		
		class FFullObjectFormatWriter : public FObjectWriter
		{
			const FFullObjectFormat::FAllowPropertyFunc& IsPropertyAllowedFunc;
		public:

			FFullObjectFormatWriter(TArray<uint8>& Data, const FFullObjectFormat::FAllowPropertyFunc& IsPropertyAllowedFunc)
				: FObjectWriter(Data)
				, IsPropertyAllowedFunc(IsPropertyAllowedFunc)
			{
				constexpr bool bIsLoading = false;
				ApplySharedArchiveConfig(*this, bIsLoading);
			}

			virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
			{
				const bool bIsAllowed = IsPropertyAllowedFunc(GetSerializedPropertyChain(), *InProperty);
				const bool bSkip = !bIsAllowed;
				return bSkip;
			}
		};

		class FFulObjectFormatReader : public FObjectReader
		{
			const IObjectReplicationFormat::FOnPropertyVisitedFunc& OnPrePropertySerialized;
		public:

			FFulObjectFormatReader(
				TArray<uint8>& Data, const IObjectReplicationFormat::FOnPropertyVisitedFunc& OnPrePropertySerialized UE_LIFETIMEBOUND
				)
				: FObjectReader(Data)
				, OnPrePropertySerialized(OnPrePropertySerialized)
			{
				constexpr bool bIsLoading = true;
				ApplySharedArchiveConfig(*this, bIsLoading);
			}

			virtual void PushSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty) override
			{
				FObjectReader::PushSerializedProperty(InProperty, bIsEditorOnlyProperty);
				
				if (const FArchiveSerializedPropertyChain* PropertyChain = GetSerializedPropertyChain())
				{
					OnPrePropertySerialized(*PropertyChain, *InProperty);
				}
				else
				{
					// Value returned by GetSerializedPropertyChain is only set once serialization has gone into a non-root property...
					// Callback takes const FArchiveSerializedPropertyChain& to simplify API user code.
					FArchiveSerializedPropertyChain Dummy;
					OnPrePropertySerialized(Dummy, *InProperty);
				}
			}
		};
	}
	
	TOptional<FConcertSessionSerializedPayload> FFullObjectFormat::CreateReplicationEvent(UObject& Object, FAllowPropertyFunc IsPropertyAllowedFunc)
	{
		FFullObjectReplicationData Data;
		FullObjectFormat::FFullObjectFormatWriter Writer(Data.SerializedObjectData.Bytes, IsPropertyAllowedFunc);
		constexpr bool bInLoadIfFindFails = false;
		// TODO: This is really bad for network travel. Names and paths should be mapped to an uint32. The mapping data is sent by the client once and then reused in the future. 
		FObjectAndNameAsStringProxyArchive ObjectAndNameProxy(Writer, bInLoadIfFindFails);
		Object.Serialize(ObjectAndNameProxy);

		FConcertSessionSerializedPayload Payload;
		Payload.SetTypedPayload(Data);
		return Payload;
	}

	void FFullObjectFormat::CombineReplicationEvents(FConcertSessionSerializedPayload& Base, const FConcertSessionSerializedPayload& Newer)
	{
		// Full object format always serializes all data. Hence the combination of some Base and Newer is just Newer.
		// Base and Newer should have the similar size. That means that Base's FConcertSessionSerializedPayload::FConcertByteArray operator= should hopefully reuse the memory allocated for TArray<uint8>
		Base = Newer; 
	}

	void FFullObjectFormat::ApplyReplicationEvent(UObject& Object, const FConcertSessionSerializedPayload& Payload, const FOnPropertyVisitedFunc& OnPrePropertySerialized)
	{
		// This can be quite some data... maybe reading could be done without creating a copy...
		FFullObjectReplicationData FullObjectFormat;
		Payload.GetTypedPayload(FullObjectFormat);

		FullObjectFormat::FFulObjectFormatReader WriterArchive(FullObjectFormat.SerializedObjectData.Bytes, OnPrePropertySerialized);
		constexpr bool bInLoadIfFindFails = false;
		FObjectAndNameAsStringProxyArchive ObjectAndNameProxy(WriterArchive, bInLoadIfFindFails);
		Object.Serialize(ObjectAndNameProxy);
	}
}
