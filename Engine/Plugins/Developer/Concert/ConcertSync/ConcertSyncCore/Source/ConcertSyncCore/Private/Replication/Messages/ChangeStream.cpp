// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Messages/ChangeStream.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChangeStream)

TOptional<FConcertReplication_ChangeStream_PutObject> FConcertReplication_ChangeStream_PutObject::MakeFromInfo(const FConcertReplicatedObjectInfo& New)
{
	if (!New.IsValidForSendingToServer())
	{
		return {};
	}
	return FConcertReplication_ChangeStream_PutObject{ New.PropertySelection, New.ClassPath };
}

TOptional<FConcertReplication_ChangeStream_PutObject> FConcertReplication_ChangeStream_PutObject::MakeFromChange(const FConcertReplicatedObjectInfo& Base, const FConcertReplicatedObjectInfo& Desired)
{
	if (Base.IsValidForSendingToServer() && Desired.IsValidForSendingToServer())
	{
		const FConcertPropertySelection PropertySelection = Base.PropertySelection != Desired.PropertySelection ? Desired.PropertySelection : FConcertPropertySelection{};
		const FSoftClassPath ClassPath = Base.ClassPath != Desired.ClassPath ? Desired.ClassPath : FSoftClassPath{};
		return FConcertReplication_ChangeStream_PutObject{ PropertySelection, ClassPath };
	}
	return {};
}

TOptional<FConcertReplicatedObjectInfo> FConcertReplication_ChangeStream_PutObject::MakeObjectInfoIfValid() const
{
	if (Properties.ReplicatedProperties.IsEmpty() || ClassPath.IsNull())
	{
		return {};
	}
	return FConcertReplicatedObjectInfo{ ClassPath, Properties };
}
