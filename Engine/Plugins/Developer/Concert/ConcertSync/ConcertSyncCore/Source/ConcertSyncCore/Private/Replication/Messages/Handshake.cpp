// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Messages/Handshake.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Handshake)

namespace UE::ConcertSyncCore::Replication
{
	FString LexJoinErrorCode(EJoinReplicationErrorCode ErrorCode)
	{
		static_assert(static_cast<int32>(EJoinReplicationErrorCode::Max) == 8, "Update JoinErrorCodeToText when you change entries");

		switch (ErrorCode)
		{
		case EJoinReplicationErrorCode::Success: return TEXT("Success");
		case EJoinReplicationErrorCode::NetworkError: return TEXT("NetworkError");
		case EJoinReplicationErrorCode::Cancelled: return TEXT("Cancelled");
		case EJoinReplicationErrorCode::AlreadyInProgress: return TEXT("AlreadyInProgress");
		case EJoinReplicationErrorCode::NotInAnyConcertSession: return TEXT("NotInAnyConcertSession");
		case EJoinReplicationErrorCode::InvalidClass: return TEXT("InvalidClass");
		case EJoinReplicationErrorCode::AlreadyInSession: return TEXT("AlreadyInSession");
		case EJoinReplicationErrorCode::DuplicateStreamId: return TEXT("DuplicateStreamId");
		case EJoinReplicationErrorCode::FailedToUnpackStream: return TEXT("FailedToUnpackStream");
		default: 
			return FString();
		}
	}
}
