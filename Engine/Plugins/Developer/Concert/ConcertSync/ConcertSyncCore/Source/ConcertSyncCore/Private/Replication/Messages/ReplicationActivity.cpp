// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Messages/ReplicationActivity.h"

#include "Internationalization/Text.h"

#include <type_traits>

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicationActivity)

#define LOCTEXT_NAMESPACE "ReplicationActivity"

namespace UE::ConcertSyncCore::ReplicationActivity::Private
{
	template<typename TPayload>
	concept CPayloadRetrievable = requires (const FConcertSyncReplicationEvent& Event, TPayload&& Payload){ Event.GetPayload(Payload); };
	
	template<CPayloadRetrievable TPayload> 
	static bool IsReplicationPayloadEqual(const FConcertSyncReplicationEvent& Left, const FConcertSyncReplicationEvent& Right)
	{
		TPayload LeftContent;
		TPayload RightContent;
		const bool bLeftSucceeded = Left.GetPayload(LeftContent);
		const bool bRightSucceeded = Right.GetPayload(RightContent);
		return bLeftSucceeded && bRightSucceeded && LeftContent == RightContent;
	}

	template<CPayloadRetrievable TPayload, typename TSummary> requires std::is_constructible_v<TSummary, const TPayload&>
	static void FillSummary(const FConcertSyncReplicationEvent& InEvent, FConcertSyncReplicationActivitySummary& Summary)
	{
		TPayload EventPayload;
		InEvent.GetPayload(EventPayload);
			
		const TSummary SummaryPayload(EventPayload);
		Summary.Payload.SetTypedPayload(SummaryPayload);
	}
	
	static FText GetDisplayText(const FConcertSyncReplicationSummary_Mute& SummaryData)
	{
		const bool bHasMuted = !SummaryData.Request.ObjectsToMute.IsEmpty();
		const bool bHasUnmuted = !SummaryData.Request.ObjectsToUnmute.IsEmpty();
		if (bHasMuted && bHasUnmuted)
		{
			return LOCTEXT("Title.Mute.PauseAndResume", "Pause & Resume");
		}

		if (bHasMuted)
		{
			return LOCTEXT("Title.Mute.Pause", "Pause replication");
		}
			
		if (bHasUnmuted)
		{
			return LOCTEXT("Title.Mute.Resume", "Resume replication");
		}

		return LOCTEXT("Title.Mute.Empty", "Pause / Resume (empty)");
	}

	template<typename TSummary>
	concept CSummaryGettable = requires (const FConcertSyncReplicationActivitySummary& Summary, TSummary& Payload) { Summary.GetSummaryData(Payload); };

	template<CSummaryGettable TSummary> 
	static FText GetSummaryText(const FConcertSyncReplicationActivitySummary& Summary, FText NoSummaryText)
	{
		TSummary SummaryData;
		return Summary.GetSummaryData(SummaryData)
			? GetDisplayText(SummaryData) // Take advantage of overloading
			: NoSummaryText;
	}
}

bool operator==(const FConcertSyncReplicationEvent& Left, const FConcertSyncReplicationEvent& Right)
{
	if (Left.ActivityType != Right.ActivityType)
	{
		return false;
	}
	
	using namespace UE::ConcertSyncCore::ReplicationActivity::Private;
	static_assert(static_cast<int32>(EConcertSyncReplicationActivityType::Count) == 3, "If you added an EConcertSyncReplicationActivityType entry, update this switch");
	switch (Left.ActivityType)
	{
	case EConcertSyncReplicationActivityType::None: return true;
	case EConcertSyncReplicationActivityType::LeaveReplication: return IsReplicationPayloadEqual<FConcertSyncReplicationPayload_LeaveReplication>(Left, Right);
	case EConcertSyncReplicationActivityType::Mute: return IsReplicationPayloadEqual<FConcertSyncReplicationPayload_Mute>(Left, Right);
	default: checkNoEntry(); return false;
	}
}

FConcertSyncReplicationActivitySummary FConcertSyncReplicationActivitySummary::CreateSummaryForEvent(const FConcertSyncReplicationEvent& InEvent)
{
	FConcertSyncReplicationActivitySummary Summary;
	Summary.ActivityType = InEvent.ActivityType;

	using namespace UE::ConcertSyncCore::ReplicationActivity::Private;
	static_assert(static_cast<uint8>(EConcertSyncReplicationActivityType::Count) == 3, "If you changed the enum entries, update this switch");
	switch (Summary.ActivityType)
	{
	case EConcertSyncReplicationActivityType::LeaveReplication:
		FillSummary<FConcertSyncReplicationPayload_LeaveReplication, FConcertSyncReplicationSummary_LeaveReplication>(InEvent, Summary);
		break;
	case EConcertSyncReplicationActivityType::Mute:
		FillSummary<FConcertSyncReplicationPayload_Mute, FConcertSyncReplicationSummary_Mute>(InEvent, Summary);
		break;
	case EConcertSyncReplicationActivityType::None: [[fallthrough]];
	default: checkNoEntry(); break;
	}
	
	return Summary;
}

FText FConcertSyncReplicationActivitySummary::ToDisplayTitle() const
{
	using namespace UE::ConcertSyncCore::ReplicationActivity::Private;
	static_assert(static_cast<uint8>(EConcertSyncReplicationActivityType::Count) == 3, "If you changed the enum entries, update this switch");
	switch (ActivityType)
	{
	case EConcertSyncReplicationActivityType::LeaveReplication: return LOCTEXT("Title.LeftReplication", "Left Replication");
	case EConcertSyncReplicationActivityType::Mute:
		return GetSummaryText<FConcertSyncReplicationSummary_Mute>(
			*this,
			 LOCTEXT("Title.Mute.FailedToGetData", "Pause / Resume")
			 );
	case EConcertSyncReplicationActivityType::None: [[fallthrough]];
	default: checkNoEntry(); return FText::GetEmpty();
	}
}

#undef LOCTEXT_NAMESPACE
