// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Misc/ReplicatedObjectHierarchyCache.h"

#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/Messages/Handshake.h"

#include "Algo/AnyOf.h"

namespace UE::ConcertSyncCore
{
	void FReplicatedObjectHierarchyCache::Clear()
	{
		ObjectMetaData.Reset();
		FObjectPathHierarchy::Clear();
	}

	bool FReplicatedObjectHierarchyCache::IsObjectReferencedDirectly(const FSoftObjectPath& ObjectPath, TConstArrayView<FGuid> IgnoredClients) const
	{
		const FObjectMetaData* MetaData = ObjectMetaData.Find(ObjectPath);
		return MetaData && Algo::AnyOf(MetaData->ReferencingStreams, [&IgnoredClients](const FStreamReferencer& Referencer)
		{
			const bool bIsIgnored = IgnoredClients.Contains(Referencer.ClientId);
			return !bIsIgnored;
		});
	}
	
	void FReplicatedObjectHierarchyCache::OnJoin(const FGuid& ClientId, const FConcertReplication_Join_Request& Request)
	{
		for (const FConcertReplicationStream& Stream : Request.Streams)
		{
			const FGuid& StreamId = Stream.BaseDescription.Identifier;
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ObjectInfo : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				AddObjectInternal({ { StreamId, ObjectInfo.Key }, ClientId });
			}
		}
	}

	void FReplicatedObjectHierarchyCache::OnPostClientLeft(const FGuid& ClientId, TConstArrayView<FConcertReplicationStream> Streams)
	{
		for (const FConcertReplicationStream& Stream : Streams)
		{
			const FGuid& StreamId = Stream.BaseDescription.Identifier;
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ObjectInfo : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				RemoveObjectInternal({ { StreamId, ObjectInfo.Key }, ClientId });
			}
		}
	}

	void FReplicatedObjectHierarchyCache::OnChangeStreams(const FGuid& ClientId, TConstArrayView<FConcertObjectInStreamID> AddedObjects, TConstArrayView<FConcertObjectInStreamID> RemovedObjects)
	{
		for (const FConcertObjectInStreamID& RemovedObject : RemovedObjects)
		{
			RemoveObjectInternal({ { RemovedObject }, ClientId });
		}

		for (const FConcertObjectInStreamID& AddeddObject : AddedObjects)
		{
			AddObjectInternal({ { AddeddObject }, ClientId });
		}
	}

	void FReplicatedObjectHierarchyCache::AddObjectInternal(const FConcertReplicatedObjectId& ObjectInfo)
	{
		AddObject(ObjectInfo.Object);
		
		ObjectMetaData.FindOrAdd(ObjectInfo.Object)
			.ReferencingStreams.AddUnique({ ObjectInfo.SenderEndpointId, ObjectInfo.StreamId });
	}

	void FReplicatedObjectHierarchyCache::RemoveObjectInternal(const FConcertReplicatedObjectId& Object)
	{
		if (RemoveMetaData(Object))
		{
			RemoveObject(Object.Object);
		}
	}

	bool FReplicatedObjectHierarchyCache::RemoveMetaData(const FConcertReplicatedObjectId& Object)
	{
		FObjectMetaData* MetaData = ObjectMetaData.Find(Object.Object);
		// Every AddObjectInternal call should be matched with a RemoveObjectInternal call
		if (!ensure(MetaData))
		{
			return false;
		}
		
		const int32 Index = MetaData->ReferencingStreams.IndexOfByPredicate([&Object](const FStreamReferencer& Referencer)
		{
			return Referencer.ClientId == Object.SenderEndpointId && Referencer.StreamId == Object.StreamId;
		});
		const bool bIsValidIndex = MetaData->ReferencingStreams.IsValidIndex(Index);
		if (bIsValidIndex && MetaData->ReferencingStreams.Num() == 1)
		{
			ObjectMetaData.Remove(Object.Object);
			return true;
		}

		if (bIsValidIndex)
		{
			MetaData->ReferencingStreams.RemoveAtSwap(Index);
		}
		return false;
	}
}
