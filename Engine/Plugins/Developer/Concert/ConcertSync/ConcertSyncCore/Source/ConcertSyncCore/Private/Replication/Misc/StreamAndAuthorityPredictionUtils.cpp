// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"

#include "ConcertMessageData.h"
#include "Replication/Messages/ReplicationActivity.h"

#include "Algo/Transform.h"
#include "Templates/Function.h"

#include <type_traits>

namespace UE::ConcertSyncCore::Replication
{
	namespace Private
	{
		static bool HandleLeaveReplicationEvent(
			const FConcertSyncReplicationEvent& ReplicationEvent,
			TArray<FConcertBaseStreamInfo>& OutPredictedStream,
			TArray<FConcertObjectInStreamID>& OutPredictedAuthority
			)
		{
			FConcertSyncReplicationPayload_LeaveReplication LeaveReplication;
			if (!ReplicationEvent.GetPayload(LeaveReplication))
			{
				return false;
			}

			Algo::Transform(LeaveReplication.Streams, OutPredictedStream, [](const FConcertReplicationStream& Stream){ return Stream.BaseDescription; });
			OutPredictedAuthority = MoveTemp(LeaveReplication.OwnedObjects);
			return true;
		}

		static bool AnalyzeActivity(
			const FConcertSyncActivity& Activity,
			TFunctionRef<bool(const FGuid& EndpointId)> IsTargetEndpointFunc,
			FExtractReplicationEventFunc ExtractReplicationEventFunc,
			TArray<FConcertBaseStreamInfo>& OutPredictedStream,
			TArray<FConcertObjectInStreamID>& OutPredictedAuthority
			)
		{
			if (Activity.EventType != EConcertSyncActivityEventType::Replication
				|| !IsTargetEndpointFunc(Activity.EndpointId)
				|| Activity.bIgnored)
			{
				return false;
			}

			bool bSuccess = false;
			const uint64 EventId = Activity.EventId;
			ExtractReplicationEventFunc(EventId, [&OutPredictedStream, &OutPredictedAuthority, &bSuccess](const FConcertSyncReplicationEvent& Event)
			{
				bSuccess = Event.ActivityType == EConcertSyncReplicationActivityType::LeaveReplication
					&& HandleLeaveReplicationEvent(Event, OutPredictedStream, OutPredictedAuthority);
			});
			return bSuccess;
		}

		/** This indirection avoids Algo::Transform alloc overhead that would be required by the BacktrackActivityHistoryForActivityThatSetsContent overloads. */
		template<typename TActivityType, typename TExtractLambda> requires std::is_invocable_r_v<const FConcertSyncActivity&, TExtractLambda, int32>
		static TOptional<int64> BacktrackActivityHistoryGeneric(
			TConstArrayView<TActivityType> Activities,
			TFunctionRef<bool(const FGuid& EndpointId)> IsTargetEndpointFunc,
			FExtractReplicationEventFunc GetReplicationEventFunc,
			TArray<FConcertBaseStreamInfo>& OutStreams,
			TArray<FConcertObjectInStreamID>& OutAuthority,
			TExtractLambda&& ExtractLambda
			)
		{
			// Walk backwards since we're looking for the latest activity that SET our client's state.
			for (int32 i = Activities.Num() - 1; i >= 0; --i)
			{
				const FConcertSyncActivity& Activity = ExtractLambda(i);
				if (AnalyzeActivity(Activity, IsTargetEndpointFunc, GetReplicationEventFunc, OutStreams, OutAuthority))
				{
					return Activity.ActivityId;
				}
			}
			return {};
		}
	}

	TOptional<int64> BacktrackActivityHistoryForActivityThatSetsContent(
		TConstArrayView<FConcertSyncActivity> Activities,
		TFunctionRef<bool(const FGuid& EndpointId)> IsTargetEndpointFunc,
		FExtractReplicationEventFunc GetReplicationEventFunc,
		TArray<FConcertBaseStreamInfo>& OutStreams,
		TArray<FConcertObjectInStreamID>& OutAuthority
		)
	{
		return Private::BacktrackActivityHistoryGeneric(Activities, IsTargetEndpointFunc, GetReplicationEventFunc, OutStreams, OutAuthority,
			[&Activities](int32 i){ return Activities[i]; }
		);
	}

	TOptional<int64> BacktrackActivityHistoryForActivityThatSetsContent(
		TConstArrayView<FConcertSessionActivity> Activities,
		TFunctionRef<bool(const FGuid& EndpointId)> IsTargetEndpointFunc,
		FExtractReplicationEventFunc GetReplicationEventFunc,
		TArray<FConcertBaseStreamInfo>& OutStreams,
		TArray<FConcertObjectInStreamID>& OutAuthority
		)
	{
		return Private::BacktrackActivityHistoryGeneric(Activities, IsTargetEndpointFunc, GetReplicationEventFunc, OutStreams, OutAuthority,
			[&Activities](int32 i){ return Activities[i].Activity; }
		);
	}

	bool AreLogicallySameClients(const FConcertClientInfo& First, const FConcertClientInfo& Second)
	{
		return First.DisplayName == Second.DisplayName && First.DeviceName == Second.DeviceName;
	}
}
