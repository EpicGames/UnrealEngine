// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/MuteUtils.h"

#include "Replication/Messages/Muting.h"

namespace UE::ConcertSyncCore::Replication::MuteUtils
{
	namespace Private
	{
		static void ProcessObjectsToMute(
			FConcertReplication_ChangeMuteState_Request& InOutBase,
			const FConcertReplication_ChangeMuteState_Request& InRequestToMerge,
			const IMuteStateGroundTruth& InGroundTruth
			)
		{
			for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& MuteRequest : InRequestToMerge.ObjectsToMute)
			{
				const FSoftObjectPath& ObjectPath = MuteRequest.Key;
				if (!InGroundTruth.IsObjectKnown(ObjectPath))
				{
					continue;
				}
				
				const FConcertReplication_ObjectMuteSetting& MuteSetting = MuteRequest.Value;
				const EMuteState MuteState = InGroundTruth.GetMuteState(ObjectPath);
				// Already in base request? Then we have nothing to add.
				if (const FConcertReplication_ObjectMuteSetting* BaseSetting = InOutBase.ObjectsToMute.Find(ObjectPath)
					; MuteState == EMuteState::ExplicitlyMuted && BaseSetting && *BaseSetting == MuteSetting)
				{
					continue;
				}

				// InRequestToMerge wants the object to be muted so remove the unmute operation
				InOutBase.ObjectsToUnmute.Remove(ObjectPath);
				
				// Already the status quo? Then it's a pointless request.
				const TOptional<FConcertReplication_ObjectMuteSetting> CurrentState = InGroundTruth.GetExplicitSetting(ObjectPath);
				const bool bIsImplicit = !CurrentState;
				const bool bChangesExplicitState = CurrentState && *CurrentState != MuteSetting; 
				if (bIsImplicit || bChangesExplicitState)
				{
					InOutBase.ObjectsToMute.Add(ObjectPath, MuteSetting);
				}
			}
		}

		static void ProcessObjectsToUnmute(
			FConcertReplication_ChangeMuteState_Request& InOutBase,
			const FConcertReplication_ChangeMuteState_Request& InRequestToMerge,
			const IMuteStateGroundTruth& InGroundTruth
			)
		{
			for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& MuteRequest : InRequestToMerge.ObjectsToUnmute)
			{
				const FSoftObjectPath& ObjectPath = MuteRequest.Key;
				if (!InGroundTruth.IsObjectKnown(ObjectPath))
				{
					continue;
				}

				const FConcertReplication_ObjectMuteSetting& MuteSetting = MuteRequest.Value;
				const EMuteState MuteState = InGroundTruth.GetMuteState(ObjectPath);
				// Already in base request? Then we have nothing to add.
				if (const FConcertReplication_ObjectMuteSetting* BaseSetting = InOutBase.ObjectsToUnmute.Find(ObjectPath)
					; MuteState == EMuteState::ExplicitlyUnmuted && BaseSetting && *BaseSetting == MuteSetting)
				{
					continue;
				}
				
				// InRequestToMerge wants the object to be unmuted so remove the mute operation
				InOutBase.ObjectsToMute.Remove(ObjectPath);
				
				// The request cannot unmute the object if is not affected by any mute effect.
				if (MuteState == EMuteState::None)
				{
					continue;
				}
				
				// Already the status quo? Then it's a pointless request.
				const TOptional<FConcertReplication_ObjectMuteSetting> CurrentState = InGroundTruth.GetExplicitSetting(ObjectPath);
				const bool bIsImplicit = !CurrentState;
				const bool bChangesExplicitState = CurrentState && *CurrentState != MuteSetting; 
				if (bIsImplicit || bChangesExplicitState)
				{
					InOutBase.ObjectsToUnmute.Add(ObjectPath, MuteSetting);
				}
			}
		}
	}
	
	void CombineMuteRequests(
		FConcertReplication_ChangeMuteState_Request& InOutBase,
		const FConcertReplication_ChangeMuteState_Request& InRequestToMerge,
		const IMuteStateGroundTruth& InGroundTruth
		)
	{
		Private::ProcessObjectsToMute(InOutBase, InRequestToMerge, InGroundTruth);
		Private::ProcessObjectsToUnmute(InOutBase, InRequestToMerge, InGroundTruth);
	}
}