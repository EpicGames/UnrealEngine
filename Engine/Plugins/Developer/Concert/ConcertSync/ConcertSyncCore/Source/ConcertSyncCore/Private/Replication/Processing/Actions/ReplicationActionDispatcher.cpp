// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Processing/Actions/ReplicationActionDispatcher.h"

#include "ConcertLogGlobal.h"
#include "Replication/Data/ReplicationActionEntry.h"

namespace UE::ConcertSyncCore
{
FReplicationActionDispatcher::FReplicationActionDispatcher(const TArray<FConcertReplicationActionEntry>& InActions, bool bDebugActions)
	: Actions(InActions)
	, bDebugActions(bDebugActions)
	, ActionsToTrigger(false, InActions.Num())
{}

void FReplicationActionDispatcher::OnReplicateProperty(const FProperty& InProperty)
{
	for (int32 Index = 0; Index < Actions.Num(); ++Index)
	{
		const FConcertReplicationActionEntry& Entry = Actions[Index];
		const FConcertReplicationAction* Action = Actions[Index].Action.GetPtr<FConcertReplicationAction>();
		if (!Action || ActionsToTrigger[Index])
		{
			continue;
		}

		const TArray<TFieldPath<FProperty>>& Properties = Entry.Properties;
		const int32 PropertyIndex = Properties.IndexOfByPredicate([&InProperty](const TFieldPath<FProperty>& FieldPath)
		{
			const FProperty* ResolvedProperty = FieldPath.Get();
			return ResolvedProperty == &InProperty;
		});
		const bool bContainsProperty = Properties.IsValidIndex(PropertyIndex);
		if (bContainsProperty)
		{
			ActionsToTrigger[Index] = true;
			UE_CLOGF(bDebugActions, LogConcert, Log, "Property %ls executes action %ls",
				*Properties[PropertyIndex].ToString(),
				*Entry.Action.GetScriptStruct()->GetName()
				);
		}
	}
}

void FReplicationActionDispatcher::ExecuteActions(const FReplicationActionArgs& InArgs)
{
	check(Actions.Num() == ActionsToTrigger.Num()); // We assume config has not changed since construction.

	for (int32 Index = 0; Index < Actions.Num(); ++Index)
	{
		const FConcertReplicationAction* Action = Actions[Index].Action.GetPtr<FConcertReplicationAction>();
		if (ActionsToTrigger[Index] && Action)
		{
			Action->Apply(InArgs);
		}
	}
}
}
