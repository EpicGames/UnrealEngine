// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Processing/ObjectReplicationCache.h"

#include "Replication/Data/ObjectIds.h"
#include "Replication/Formats/IObjectReplicationFormat.h"
#include "Replication/Messages/ObjectReplication.h"

namespace UE::ConcertSyncCore
{
	FObjectReplicationCache::FObjectReplicationCache(IObjectReplicationFormat& ReplicationFormat)
		: ReplicationFormat(ReplicationFormat)
	{}

	FCacheStoreStats FObjectReplicationCache::StoreUntilConsumed(const FGuid& SendingEndpointId, const FGuid& OriginStreamId, const FSequenceId SequenceId, const FConcertReplication_ObjectReplicationEvent& ObjectReplicationEvent)
	{
		FCacheStoreStats Stats;
		
		const FConcertReplicatedObjectId ObjectId{ { OriginStreamId, ObjectReplicationEvent.ReplicatedObject }, SendingEndpointId };
		FObjectCache* ObjectCacheBeforeAddition = Cache.Find(ObjectId);
		if (ObjectCacheBeforeAddition)
		{
			Stats.NumCacheUpdates = CombineCachedDataWithNewData(ObjectId, SequenceId, ObjectReplicationEvent, *ObjectCacheBeforeAddition);
		}

		FObjectCache* ObjectCacheAfterAddition = ObjectCacheBeforeAddition ? ObjectCacheBeforeAddition : nullptr;
		TSharedPtr<FConcertReplication_ObjectReplicationEvent> LazilyCopiedEventPtr;
		for (const TSharedRef<IReplicationCacheUser>& CacheUser : CacheUsers)
		{
			if (ObjectCacheBeforeAddition
				// They've already had OnDataCached called if they're contained
				&& ObjectCacheBeforeAddition->DataInUse.Contains(CacheUser))
			{
				continue;
			}

			if (CacheUser->WantsToAcceptObject(ObjectId))
			{
				++Stats.NumInsertions;
				
				// Do the event copy only when somebody wants the data...
				if (!LazilyCopiedEventPtr.IsValid())
				{
					LazilyCopiedEventPtr = MakeShared<FConcertReplication_ObjectReplicationEvent>(ObjectReplicationEvent);
				}

				// We'll constructor a new proxy shared ptr which will clean-up Cache when released
				TWeakPtr<IReplicationCacheUser> WeakUserPtr = CacheUser;
				TWeakPtr<FObjectReplicationCache> WeakThisPtr = AsWeak();
				TSharedRef<FConcertReplication_ObjectReplicationEvent> GuardPtr = MakeShareable(LazilyCopiedEventPtr.Get(), [OriginStreamId, LazilyCopiedEventPtr, WeakUserPtr, WeakThisPtr](auto*)
				{
					// Replication cache user can survive the destruction of the cache because it is created externally
					const TSharedPtr<FObjectReplicationCache> This = WeakThisPtr.Pin();
					if (!This)
					{
						return;
					}

					// ObjectCache may not be found because cache user was unregistered and then destroyed: UnregisterDataCacheUser removes cache users.
					const FConcertObjectInStreamID ObjectId{ OriginStreamId, LazilyCopiedEventPtr->ReplicatedObject };
					FObjectCache* ObjectCache = This->Cache.Find(ObjectId);
					if (!ObjectCache)
					{
						return;
					}
					
					ObjectCache->DataInUse.Remove(WeakUserPtr);
					if (ObjectCache->DataInUse.IsEmpty())
					{
						This->Cache.Remove(ObjectId);
					}
					
					// LazyCopiedEventPtr will decrement counter and possibly be destroyed now
				});
				
				const TWeakPtr<FConcertReplication_ObjectReplicationEvent> WeakGuardPtr = GuardPtr;
				CacheUser->OnDataCached(ObjectId, SequenceId, MoveTemp(GuardPtr));
				// Was it instantly consumed? Should not really happen but it could technically...
				if (!LIKELY(WeakGuardPtr.IsValid()))
				{
					continue;
				}
				
				if (!ObjectCacheAfterAddition)
				{
					ObjectCacheAfterAddition = &Cache.Add(ObjectId);
				}
				ObjectCacheAfterAddition->DataInUse.Add(WeakUserPtr, WeakGuardPtr);
			}
		}

		return Stats;
	}

	void FObjectReplicationCache::RegisterDataCacheUser(TSharedRef<IReplicationCacheUser> User)
	{
		if (!CacheUsers.Contains(User))
		{
			CacheUsers.Emplace(MoveTemp(User));
		}
	}

	void FObjectReplicationCache::UnregisterDataCacheUser(const TSharedRef<IReplicationCacheUser>& User)
	{
		CacheUsers.RemoveSingle(User);
		
		for (auto It = Cache.CreateIterator(); It; ++It)
		{
			FObjectCache& ObjectCache = It->Value;
			ObjectCache.DataInUse.Remove(User);
			if (ObjectCache.DataInUse.IsEmpty())
			{
				It.RemoveCurrent();
			}
		}
	}

	uint32 FObjectReplicationCache::CombineCachedDataWithNewData(
		const FConcertReplicatedObjectId& ObjectId,
		FSequenceId NewSequenceId,
		const FConcertReplication_ObjectReplicationEvent& NewData,
		const FObjectCache& ObjectCacheBeforeAddition
		) const
	{
		uint32 NumUpdates = 0;
		
		// Caches users may point to the same or different memory depending how fast they process
		TSet<FConcertReplication_ObjectReplicationEvent*> CombineOnceDetection;
		for (const TPair<TWeakPtr<IReplicationCacheUser>, TWeakPtr<FConcertReplication_ObjectReplicationEvent>>& InUseDataPair : ObjectCacheBeforeAddition.DataInUse)
		{
			const TWeakPtr<IReplicationCacheUser> CacheUser = InUseDataPair.Key;
			const TWeakPtr<FConcertReplication_ObjectReplicationEvent>& EventData = InUseDataPair.Value;

			// There's no point in updating if either the user has destroyed itself (rudely without telling us) or the user has released that data already.
			const TSharedPtr<IReplicationCacheUser> CacheUserPin = CacheUser.Pin();
			const TSharedPtr<FConcertReplication_ObjectReplicationEvent> EventDataPin = EventData.Pin();
			if (!EventDataPin || !CacheUserPin)
			{
				continue;
			}

			// Multiple cache users may be using the same data so only combine once ...
			if (!CombineOnceDetection.Contains(EventDataPin.Get()))
			{
				CombineOnceDetection.Add(EventDataPin.Get());
				ReplicationFormat.CombineReplicationEvents(EventDataPin->SerializedPayload, NewData.SerializedPayload);
			}
			// ... but let every user know that the data was combined with another SequenceId
			CacheUserPin->OnCachedDataUpdated(ObjectId, NewSequenceId);

			++NumUpdates;
		}

		return NumUpdates;
	}
}
