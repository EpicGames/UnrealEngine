// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Processing/ObjectReplicationProcessor.h"

#include "ConcertMessageData.h"
#include "Replication/Processing/IReplicationDataSource.h"

namespace UE::ConcertSyncCore
{
	FObjectReplicationProcessor::FObjectReplicationProcessor(
		IReplicationDataSource& DataSource
		)
		: DataSource(DataSource)
	{}

	void FObjectReplicationProcessor::ProcessObjects(const FProcessObjectsParams& Params)
	{
		// TODO UE-190714: Respect time budget and prioritize objects
		DataSource.ConsumePendingObjects([this](const FPendingObjectReplicationInfo& ObjectInfo, IReplicationDataItem& Item)
		{
			ProcessObject(FObjectProcessArgs(ObjectInfo, Item));
		});
	}
}
