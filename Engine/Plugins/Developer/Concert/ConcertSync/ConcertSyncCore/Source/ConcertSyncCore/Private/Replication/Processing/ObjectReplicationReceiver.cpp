// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Processing/ObjectReplicationReceiver.h"

#include "ConcertLogGlobal.h"
#include "IConcertSession.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Replication/Processing/ObjectReplicationCache.h"

#include "HAL/IConsoleManager.h"
#include "Trace/ConcertProtocolTrace.h"

namespace UE::ConcertSyncCore
{
	namespace Private
	{
		static void TraceDroppedObjectEventIf(bool bSendTrace, const FConcertReplication_ObjectReplicationEvent& ObjectEvent)
		{
#if UE_CONCERT_TRACE_ENABLED
			if (bSendTrace)
			{
				CONCERT_TRACE_REPLICATION_OBJECT_SINK(Dropped, ObjectEvent.ReplicatedObject, ObjectEvent.ReplicationSequenceId);
			}
#endif
		}
	}
	
	static TAutoConsoleVariable<bool> CVarLogReceivedObjects(TEXT("Concert.Replication.LogReceivedObjects"), false, TEXT("Enable Concert logging for received replicated objects."));
	
	FObjectReplicationReceiver::FObjectReplicationReceiver(IConcertSession& Session, FObjectReplicationCache& ReplicationCache)
		: Session(Session)
		, ReplicationCache(ReplicationCache)
	{
		Session.RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(this, &FObjectReplicationReceiver::HandleBatchReplicationEvent);
	}

	FObjectReplicationReceiver::~FObjectReplicationReceiver()
	{
		Session.UnregisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(this);
	}

	void FObjectReplicationReceiver::HandleBatchReplicationEvent(const FConcertSessionContext& SessionContext, const FConcertReplication_BatchReplicationEvent& Event)
	{
		// Fyi: an object may have multiple changes in a batch replication event: each stream can modify different properties as long as they do not overlap.
		uint32 NumObjects = 0;
		uint32 NumRejectedObjectChanges = 0;
		uint32 NumCacheInsertions = 0;
		uint32 NumCacheUpdates = 0;
		uint32 NumOfAcceptedObjectChanges = 0;
		
		for (const FConcertReplication_StreamReplicationEvent& StreamEvent : Event.Streams)
		{
			const int32 ObjectsInStream = StreamEvent.ReplicatedObjects.Num();
			NumObjects += ObjectsInStream;
			
			for (const FConcertReplication_ObjectReplicationEvent& ObjectEvent : StreamEvent.ReplicatedObjects)
			{
				CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_RECEIVE(ObjectEvent.ReplicatedObject, ObjectEvent.ReplicationSequenceId);
				if (ShouldAcceptObject(SessionContext, StreamEvent, ObjectEvent))
				{
					CONCERT_TRACE_REPLICATION_OBJECT_SCOPE(EnqueueReceivedObject, ObjectEvent.ReplicatedObject, ObjectEvent.ReplicationSequenceId);
					const FCacheStoreStats CacheStoreStats = ReplicationCache.StoreUntilConsumed(SessionContext.SourceEndpointId, StreamEvent.StreamId, ObjectEvent.ReplicationSequenceId, ObjectEvent);
					NumCacheInsertions += CacheStoreStats.NumInsertions;
					NumCacheUpdates += CacheStoreStats.NumCacheUpdates;
					NumOfAcceptedObjectChanges += CacheStoreStats.NumInsertions == 0 ? 0 : 1;
					
					Private::TraceDroppedObjectEventIf(CacheStoreStats.NoChangesMade(), ObjectEvent);
				}
				else
				{
					CONCERT_TRACE_REPLICATION_OBJECT_SINK(Rejected, ObjectEvent.ReplicatedObject, ObjectEvent.ReplicationSequenceId);
					++NumRejectedObjectChanges;
				}
			}
		}

		if (CVarLogReceivedObjects.GetValueOnGameThread())
		{
			UE_LOGF(LogConcert, Log, "Received %u streams with %u object changes from endpoint %ls. Cached %u object changes with a total of new %u cache insertions and %u cache updates.",
				Event.Streams.Num(),
				NumObjects,
				*SessionContext.SourceEndpointId.ToString(),
				NumOfAcceptedObjectChanges,
				NumCacheInsertions,
				NumCacheUpdates
			);
			UE_CLOGF(NumRejectedObjectChanges > 0, LogConcert, Warning, "Rejected %d object changes.", NumRejectedObjectChanges);
		}
	}
}
