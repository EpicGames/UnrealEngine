// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Processing/ObjectReplicationSender.h"

#include "ConcertLogGlobal.h"
#include "IConcertSession.h"
#include "Replication/Processing/IReplicationDataItem.h"
#include "Replication/Processing/IReplicationDataSource.h"
#include "Trace/ConcertProtocolTrace.h"

#include "Algo/Accumulate.h"
#include "HAL/IConsoleManager.h"

namespace UE::ConcertSyncCore
{
	static TAutoConsoleVariable<bool> CVarLogSentObjects(TEXT("Concert.Replication.LogSentObjects"), false, TEXT("Enable Concert logging for sent replicated objects."));
	
	FObjectReplicationSender::FObjectReplicationSender(
		const FGuid& TargetEndpointId,
		IConcertSession& Session,
		IReplicationDataSource& DataSource
		)
		: FObjectReplicationProcessor(DataSource)
		, TargetEndpointId(TargetEndpointId)
		, Session(Session)
	{}

	void FObjectReplicationSender::ProcessObjects(const FProcessObjectsParams& Params)
	{
		FObjectReplicationProcessor::ProcessObjects(Params);

		if (!EventToSend.Streams.IsEmpty())
		{
			const int32 NumObjects = Algo::TransformAccumulate(EventToSend.Streams, [](const FConcertReplication_StreamReplicationEvent& Event){ return Event.ReplicatedObjects.Num(); }, 0);
			UE_CLOGF(CVarLogSentObjects.GetValueOnGameThread(), LogConcert, Log, "Sending %d streams with %d objects to %ls",
				EventToSend.Streams.Num(),
				NumObjects,
				*TargetEndpointId.ToString()
				);
			
			TraceStartSendingMarkedObjects(); 
			Session.SendCustomEvent(EventToSend, TargetEndpointId,
				// Replication is always unreliable - if it fails to deliver we'll send updated data soon again
				// TODO: In regular intervals send CRC values to detect that a change is missing
				EConcertMessageFlags::None
			);
			EventToSend.Streams.Empty(
				// It's not unreasonable to expect the next pass to have a similar number of objects so keep it around to avoid re-allocating all the time
				EventToSend.Streams.Num()
				);
		}
	}

	void FObjectReplicationSender::ProcessObject(const FObjectProcessArgs& Args)
	{
		// It would be easier to just CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_START here but for better precision we must postbone it until all objects have been processed
		MarkObjectForTrace(Args.ObjectInfo.ObjectId, Args.ObjectInfo.SequenceId);
		CONCERT_TRACE_REPLICATION_OBJECT_SCOPE(CollectObjectDataForSend, Args.ObjectInfo.ObjectId.Object, Args.ObjectInfo.SequenceId);
		
		const FSoftObjectPath& ReplicatedObject = Args.ObjectInfo.ObjectId.Object;
		auto CaptureData = [this, &Args, &ReplicatedObject]<typename TPayloadRefType>(TPayloadRefType&& Payload)
		{
			const int32 PreexistingIndex = EventToSend.Streams.IndexOfByPredicate([&Args](const FConcertReplication_StreamReplicationEvent& StreamData){ return StreamData.StreamId == Args.ObjectInfo.ObjectId.StreamId; });
			FConcertReplication_StreamReplicationEvent& StreamData = EventToSend.Streams.IsValidIndex(PreexistingIndex)
				? EventToSend.Streams[PreexistingIndex]
				: EventToSend.Streams[EventToSend.Streams.Emplace(Args.ObjectInfo.ObjectId.StreamId)];
			StreamData.ReplicatedObjects.Add({
				ReplicatedObject,
				static_cast<int32>(Args.ObjectInfo.SequenceId), // This is silly... Blueprints do not support uint32 so we need to pretend it's an int32.
				// Take advantage of move semantics if it is possible - this depends on how our data source internally obtains its payloads
				Forward<TPayloadRefType>(Payload)
			});
		};
		Args.Item.ExtractReplicationDataForObject(CaptureData, CaptureData);
	}

	inline void FObjectReplicationSender::MarkObjectForTrace(const FConcertReplicatedObjectId& Object, FSequenceId Id)
	{
#if UE_CONCERT_TRACE_ENABLED
		if (ConcertTrace::IsTracingReplication())
		{
			ObjectsToTraceThisFrame.Add(Object, Id);
		}
#endif
	}

	inline void FObjectReplicationSender::TraceStartSendingMarkedObjects()
	{
#if UE_CONCERT_TRACE_ENABLED
		if (ConcertTrace::IsTracingReplication())
		{
			for (const TPair<FConcertReplicatedObjectId, FSequenceId>& SendingNow : ObjectsToTraceThisFrame)
			{
				CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_START(TargetEndpointId, SendingNow.Key.Object, SendingNow.Value);
			}
			ObjectsToTraceThisFrame.Reset();
		}
#endif
	}
}
