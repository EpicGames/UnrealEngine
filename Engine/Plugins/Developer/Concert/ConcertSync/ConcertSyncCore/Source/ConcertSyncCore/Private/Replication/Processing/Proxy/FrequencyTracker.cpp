// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Processing/Proxy/FrequencyTracker.h"

#include "Replication/Data/ReplicationFrequencySettings.h"

namespace UE::ConcertSyncCore
{
	ECanProcessObject FFrequencyTracker::TrackAndCheckObject(
		const FConcertObjectInStreamID& ObjectPath,
		const FConcertObjectReplicationSettings& ReplicationSettings,
		const FPlatformSecondsTimestamp NowAsPlatformSeconds
		)
	{
		// If ObjectPath is not yet tracked, this adds a FDateTime at Tick 0, which causes it to be processed immediately.
		const FPlatformSecondsTimestamp& LastTimeProcessed = ObjectsToLastProcessed.FindOrAdd(ObjectPath);
		return ReplicationSettings.CanProcessObject(LastTimeProcessed, NowAsPlatformSeconds);
	}

	void FFrequencyTracker::OnProcessObject(const FConcertObjectInStreamID& Object, const FPlatformSecondsTimestamp NowAsPlatformSeconds)
	{
		ObjectsToLastProcessed.FindOrAdd(Object) = NowAsPlatformSeconds;
	}

	void FFrequencyTracker::CleanseOutdatedObjects(const FPlatformSecondsTimestamp CutoffTime)
	{
		for (auto It = ObjectsToLastProcessed.CreateIterator(); It; ++It)
		{
			if (It->Value <= CutoffTime)
			{
				It.RemoveCurrent();
			}
		}
	}
	
	void FFrequencyTracker_CleanByTick::Tick(float DeltaTime, const FPlatformSecondsTimestamp NowAsPlatformSeconds)
	{
		SecondsSinceLastCleanse += DeltaTime;
		if (SecondsSinceLastCleanse >= CleanUpIntervalSeconds)
		{
			SecondsSinceLastCleanse = 0.f;
			CleanseOutdatedObjects(NowAsPlatformSeconds - CleanUpIntervalSeconds);
		}
	}
}

