// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Processing/ReplicationDataQueuer.h"

#include "ConcertLogGlobal.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Replication/Processing/IReplicationDataItem.h"
#include "Trace/ConcertProtocolTrace.h"

namespace UE::ConcertSyncCore
{
namespace DataQueuer
{
	class FDataQueuerItem : public IReplicationDataItem
	{
		TSharedPtr<const FConcertReplication_ObjectReplicationEvent> DataToApply;
		TFunctionRef<void()> RemoveItemFunc;
		bool bAlreadyExtracted = false;
	public:
		
		explicit FDataQueuerItem(
			const TSharedPtr<const FConcertReplication_ObjectReplicationEvent>& DataToApply, 
			const TFunctionRef<void()>& RemoveItemFunc
			)
			: DataToApply(DataToApply)
			, RemoveItemFunc(RemoveItemFunc)
		{}

		virtual void ExtractReplicationDataForObject(
			TFunctionRef<void(const FConcertSessionSerializedPayload& Payload)> ProcessCopyable,
			TFunctionRef<void(FConcertSessionSerializedPayload&& Payload)> ProcessMoveable
			) override
		{
			if (!ensure(!bAlreadyExtracted))
			{
				return;
			}
			bAlreadyExtracted = true;
			
			RemoveItemFunc();
			// Event may be shared by other FReplicationDataQueuers since it originates from the replication cache, so sadly no move.
			ProcessCopyable(DataToApply->SerializedPayload);
		}
	};
}
	void FReplicationDataQueuer::ConsumePendingObjects(
		TFunctionRef<void(const FPendingObjectReplicationInfo& Info, IReplicationDataItem& Item)> InProcess
		)
	{
		for (auto It = PendingEvents.CreateIterator(); It; ++It)
		{
			const FPendingObjectReplicationInfo Info{ It->Key, It->Value.SequenceId };
			const auto RemoveItem = [&It]{ It.RemoveCurrent(); };
			DataQueuer::FDataQueuerItem Item(It->Value.DataToApply, RemoveItem);
			InProcess(Info, Item);
		}
	}

	void FReplicationDataQueuer::OnDataCached(const FConcertReplicatedObjectId& Object, FSequenceId SequenceId, TSharedRef<const FConcertReplication_ObjectReplicationEvent> Data)
	{
		PendingEvents.Add(Object, FPendingObjectData{ Data, SequenceId });
	}

	void FReplicationDataQueuer::OnCachedDataUpdated(const FConcertReplicatedObjectId& Object, FSequenceId SequenceId)
	{
		FPendingObjectData* ObjectData = PendingEvents.Find(Object);
		if (ensureMsgf(ObjectData, TEXT("OnCachedDataUpdated called for %s even though we have no cached data. Investigate in correct API call!"), *Object.ToString()))
		{
			// TODO DP: This may want to be a specialized macro so Insights can visually highlight that a packet was merged.
			// Trace that the old data has "finished" sending ... 
			CONCERT_TRACE_REPLICATION_OBJECT_SINK(Merged, Object.Object, ObjectData->SequenceId);
			// ... since it is merged
			ObjectData->SequenceId = SequenceId;
		}
	}

	void FReplicationDataQueuer::BindToCache(FObjectReplicationCache& InReplicationCache)
	{
		ReplicationCache = &InReplicationCache;
		ReplicationCache->RegisterDataCacheUser(AsShared());
	}
}
