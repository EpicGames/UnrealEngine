// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/PropertyChainUtils.h"

#include "ConcertLogGlobal.h"
#include "Misc/ScopeExit.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/UnrealType.h"

namespace UE::ConcertSyncCore::PropertyChain
{
	namespace Private
	{
		EBreakBehavior VisitPropertyRecursive(FArchiveSerializedPropertyChain& Chain, FProperty& Property, TFunctionRef<EBreakBehavior(const FArchiveSerializedPropertyChain&, FProperty&)> ProcessProperty);
		
		EBreakBehavior VisitStructPropertyRecursive(
			FArchiveSerializedPropertyChain& Chain,
			FStructProperty& StructProperty,
			TFunctionRef<EBreakBehavior(const FArchiveSerializedPropertyChain&, FProperty&)> ProcessProperty
			)
		{
			// If the struct is in a container, then the container should be pushed, not the inner property.
			FProperty& PropertyToPush = IsInnerContainerProperty(StructProperty)
				? *StructProperty.GetOwner<FProperty>()
				: StructProperty;
			Chain.PushProperty(&PropertyToPush, PropertyToPush.IsEditorOnlyProperty());
			ON_SCOPE_EXIT { Chain.PopProperty(&PropertyToPush, PropertyToPush.IsEditorOnlyProperty()); };
			
			for (TFieldIterator<FProperty> FieldIt(StructProperty.Struct); FieldIt; ++FieldIt)
			{
				if (VisitPropertyRecursive(Chain, **FieldIt, ProcessProperty) == EBreakBehavior::Break)
				{
					return EBreakBehavior::Break;
				}
			}
			
			return EBreakBehavior::Continue;
		}
		
		EBreakBehavior VisitPropertyRecursive(
			FArchiveSerializedPropertyChain& Chain,
			FProperty& Property,
			TFunctionRef<EBreakBehavior(const FArchiveSerializedPropertyChain&, FProperty&)> ProcessProperty
			)
		{
			if (!IsReplicatableProperty(Property))
			{
				return EBreakBehavior::Continue;
			}

			if (ProcessProperty(Chain, Property) == EBreakBehavior::Break)
			{
				return EBreakBehavior::Break;
			}
			
			if (FStructProperty* StructProperty = CastField<FStructProperty>(&Property))
			{
				/*
				 * THIS BLOCK OF TEXT CONCERNS NATIVE STRUCTS THAT DEFINE A SERIALIZE() FUNCTION.
				 * SEE TL;DR.
				 * 
				 * Note: If the struct defines a custom serialize function, it *might* not make sense to list any uproperties we find.
				 * The serialize function can skip certain properties. In those cases  exposing them to a user for selection makes no sense.
				 * When replicated, effectively the Serialize function decides what is serialized.
				 * 
				 * However: Some structs have a Serialize function just return false, which causes standard UPROPERTY serialization.
				 * Example (in 5.4): FPostProcessSettings.
				 * 
				 * The way we'll handle this is by exposing all properties anyway. 
				 * When replicating the native struct, the struct's Serialize function will then either
				 * 1. Do its custom logic, which ignores our selection set, or
				 * 2. Ask the FArchive::ShouldSerializeProperty, which will check the property select set built using this VisitPropertyRecursive function.
				 * 
				 * TL;DR: Effectively we'll expose properties (which will be displayed in UI to users) that sometimes cannot be replicated.
				 * The advantage is that at structs that implement Serialize functions can be correctly replicated. Fair trade-off.
				 */
				return VisitStructPropertyRecursive(Chain, *StructProperty, ProcessProperty);
			}

			auto HandleContainer = [&Chain, &ProcessProperty](FProperty& Inner)
			{
				// Inner properties are never listed in the path. So we only need to continue if there are more struct sub-properties.
				FStructProperty* InnerStructProperty = CastField<FStructProperty>(&Inner);
				if (!InnerStructProperty)
				{
					return EBreakBehavior::Continue;
				}
				
				return InnerStructProperty && IsReplicatableProperty(Inner)
					? VisitStructPropertyRecursive(Chain, *InnerStructProperty, ProcessProperty)
					: EBreakBehavior::Continue;
			};

			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(&Property))
			{
				return HandleContainer(*ArrayProperty->Inner);
			}

			if (const FSetProperty* SetProperty = CastField<FSetProperty>(&Property))
			{
				return HandleContainer(*SetProperty->ElementProp);
			}

			if (const FMapProperty* MapProperty = CastField<FMapProperty>(&Property))
			{
				return IsReplicatableProperty(*MapProperty->KeyProp)
					? HandleContainer(*MapProperty->ValueProp)
					: EBreakBehavior::Continue;
			}
			
			return EBreakBehavior::Continue;
		}
	}
	
	FProperty* ResolveProperty(const UStruct& Class, const FConcertPropertyChain& ChainToResolve, bool bLogOnFail)
	{
		SCOPED_CONCERT_TRACE(ResolveProperty);
		
		FProperty* Result = nullptr;
		ForEachReplicatableProperty(
			Class,
			[&ChainToResolve, &Result](const FArchiveSerializedPropertyChain& Chain, FProperty& LeafProperty) mutable
			{
				if (ChainToResolve.MatchesExactly(&Chain, LeafProperty))
				{
					Result = &LeafProperty;
					return EBreakBehavior::Break;
				}
				return EBreakBehavior::Continue;
			});
		UE_CLOGF(bLogOnFail && Result == nullptr, LogConcert, Warning, "Property chain \"%ls\" resolved to no property!", *ChainToResolve.ToString());
		return Result;
	}
	
	void ForEachReplicatableProperty(
		const UStruct& Class,
		TFunctionRef<EBreakBehavior(const FArchiveSerializedPropertyChain& Chain, FProperty& LeafProperty)> ProcessProperty)
	{
		FArchiveSerializedPropertyChain Chain;
		for (TFieldIterator<FProperty> FieldIt(&Class); FieldIt; ++FieldIt)
		{
			if (Private::VisitPropertyRecursive(Chain, **FieldIt, ProcessProperty) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	void ForEachReplicatableConcertProperty(
		const UStruct& Class,
		TFunctionRef<EBreakBehavior(FConcertPropertyChain&& PropertyChain)> ProcessProperty)
	{
		ForEachReplicatableProperty(Class, [&ProcessProperty](const FArchiveSerializedPropertyChain& Chain, FProperty& LeafProperty)
		{
			return ProcessProperty(FConcertPropertyChain(&Chain, LeafProperty));
		});
	}

	void BulkConstructConcertChainsFromPaths(const UStruct& Class, uint32 NumPaths, TFunctionRef<bool(const FArchiveSerializedPropertyChain& Chain, FProperty& LeafProperty)> MatchesPath)
	{
		if (NumPaths == 0)
		{
			return;
		}
		
		ForEachReplicatableProperty(Class, [&NumPaths, &MatchesPath](const FArchiveSerializedPropertyChain& Chain, FProperty& LeafProperty) mutable
		{
			if (MatchesPath(Chain, LeafProperty))
			{
				--NumPaths;
				return NumPaths == 0
					? EBreakBehavior::Break
					: EBreakBehavior::Continue;
			}
			return EBreakBehavior::Continue;
		});
	}
	
	bool DoPathAndChainsMatch(const TArray<FName>& Path, const FArchiveSerializedPropertyChain& Chain, const FProperty& LeafProperty)
	{
		if (Chain.GetNumProperties() + 1 != Path.Num())
		{
			return false;
		}
		
		for (int32 i = 0; i < Path.Num() - 1; ++i)
		{
			if (Path[i] != Chain.GetPropertyFromRoot(0)->GetFName())
			{
				return false;
			}
		}

		return Path[Path.Num() - 1] == LeafProperty.GetFName();
	}
	
	bool IsReplicatableProperty(const FProperty& LeafProperty)
	{
		const bool bHasValidFlags = !LeafProperty.HasAnyPropertyFlags(
			// Replicating delegates makes no sense
			CPF_BlueprintAssignable
			// It does not make sense to serialize the reference of an instanced subobject. Instead the subobject should be added to the list of replicated properties.
			| CPF_InstancedReference /* for object ptrs */ | CPF_ContainsInstancedReference /* for containers of object ptrs */
			);
		const bool bIsAllowedObjectProperty = CastField<FObjectPropertyBase>(&LeafProperty) == nullptr
			// Soft object properties are allowed because they are serialized as a string.
			|| CastField<FSoftObjectProperty>(&LeafProperty) != nullptr;
		return bHasValidFlags && bIsAllowedObjectProperty;
	}

	bool IsInnerContainerProperty(const FProperty& Property)
	{
		const FProperty* ParentProperty = Property.GetOwner<FProperty>();
		const bool bIsInnerContainerProperty = ParentProperty
			&& (ParentProperty->IsA(FArrayProperty::StaticClass()) || ParentProperty->IsA(FSetProperty::StaticClass()) || ParentProperty->IsA(FMapProperty::StaticClass()));
		return bIsInnerContainerProperty;
	}
	
	bool IsPropertyEligibleForMarkingAsInternal(const FProperty& Property)
	{
		return IsInnerContainerProperty(Property)
			&& (IsPrimitiveProperty(Property) || IsNativeStructProperty(Property));
	}

	bool IsPrimitiveProperty(const FProperty& Property)
	{
		return Property.IsA(FNumericProperty::StaticClass())
			|| Property.IsA(FBoolProperty::StaticClass())
			|| Property.IsA(FEnumProperty::StaticClass());
	}
	
	bool IsNativeStructProperty(const FProperty& Property)
	{
		return Property.IsA(FStructProperty::StaticClass())
			&& CastField<FStructProperty>(&Property)->Struct->GetCppStructOps()->HasSerializer();
	}
}
