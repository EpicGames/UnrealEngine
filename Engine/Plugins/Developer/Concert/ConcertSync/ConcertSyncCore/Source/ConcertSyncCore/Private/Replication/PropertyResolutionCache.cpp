// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/PropertyResolutionCache.h"

#include "Replication/Data/ConcertPropertySelection.h"

#include "UObject/Class.h"

namespace UE::ConcertSyncCore::PropertyChain
{
	FProperty* FPropertyResolutionCache::ResolveAndCache(const UStruct& Struct, const FConcertPropertyChain& Chain)
	{
		FClassCache& ClassCache = CachedProperties.FindOrAdd(&Struct);
		if (FProperty** const CachedProperty = ClassCache.Cache.Find(Chain))
		{
			return *CachedProperty;
		}

		FProperty* ResolvedProperty = Chain.ResolveProperty(Struct);
		UClass* OwningClass = ResolvedProperty ? ResolvedProperty->GetOwnerClass() : nullptr;
		UStruct* OwningStruct = ResolvedProperty ? ResolvedProperty->GetOwnerStruct() : nullptr;
		UScriptStruct* OwningScriptStruct = Cast<UScriptStruct>(OwningStruct);

		const bool bIsNativeClass = OwningClass && OwningClass->HasAnyClassFlags(CLASS_Native);
		const bool bIsNativeStruct = (OwningScriptStruct && OwningScriptStruct->StructFlags & STRUCT_Native)
			// Example: FVector and PRESUMABLY other classes in UObject/NoExportTypes.h
			|| (OwningStruct && !OwningScriptStruct);
		if (bIsNativeClass || bIsNativeStruct)
		{
			ClassCache.Cache.Add(Chain, ResolvedProperty);
		}
		return ResolvedProperty;
	}
}