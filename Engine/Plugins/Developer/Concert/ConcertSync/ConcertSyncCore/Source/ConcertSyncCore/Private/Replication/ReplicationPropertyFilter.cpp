// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/ReplicationPropertyFilter.h"

#include "Replication/PropertyChainUtils.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"

namespace UE::ConcertSyncCore
{
	FReplicationPropertyFilter::FReplicationPropertyFilter(const FConcertPropertySelection& PropertySelection)
		: PropertySelection(PropertySelection)
	{
		for (const FConcertPropertyChain& Chain : PropertySelection.ReplicatedProperties)
		{
			LeafToChain.FindOrAdd(Chain.GetLeafProperty()).Add(&Chain);
		}
	}

	bool FReplicationPropertyFilter::ShouldSerializeProperty(const FArchiveSerializedPropertyChain* Chain, const FProperty& Property) const
	{
		// If Property is in a container then the either it is
		//  1. primitive, in which case the property is just serialized, or 
		//  2. a struct that either has
		//		2.1 No native Serialize function: in this case we'll get recursive ShouldSerializeProperty calls.
		//		2.2 A native serialize function: In this case we may get 0 or more ShouldSerializeProperty calls.
		//		The Serialize function will just write whatever it wants. If it returns false, normal UPROPERTY serialization occurs,
		//		which means we'll end up in case 2.1 
		if (PropertyChain::IsInnerContainerProperty(Property))
		{
			if (ensureMsgf(Chain && Chain->GetNumProperties() >= 1, TEXT("Assumption broken that Property is in a container (array, set, map). Check IsPropertyEligibleForMarkingAsInternal implementation!")))
			{
				return false;
			}
			
			// We should only have gotten here because a previous call to IsPropertyInSelection(&CopiedChain, *OwnerOfProperty) returned true already (or the property would not have been pushed into the chain).
			return true;
		}
		
		const TArray<const FConcertPropertyChain*>* ChainsEndingWithPropertyName = LeafToChain.Find(Property.GetFName());
		if (!ChainsEndingWithPropertyName)
		{
			/* No chain ends with this property so it is not in the selection
			 * Note: The property selection must also contain every parent property.
			 * Example: If MyStruct.Float is in the selection, so must be MyStruct.
			 * Otherwise this will return false for properties in the middle of the chain (and it should return true).
			 */
			return false;
		}

		for (const FConcertPropertyChain* ConcertChain : *ChainsEndingWithPropertyName)
		{
			if (ConcertChain->MatchesExactly(Chain, Property))
			{
				return true;
			}
		}
		
		return false;
	}
}
