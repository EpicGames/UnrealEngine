// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertFileSharingService.h"

#define UE_API CONCERTSYNCCORE_API

/**
 * Share files using a path that is connected to a cloud based shared file service, e.g. a shared drive. It will work across all clients and server
 * assuming that the drive is visible. Validation checks are run during session connection to establish that both the server and client can see the
 * same system.
  */
class FConcertCloudFileSharingService : public IConcertFileSharingService
{
public:
	/**
	 * Construct the cloud file sharing service.
	 */
	UE_API FConcertCloudFileSharingService(const FString& Role);

	/** Destructor */
	UE_API virtual ~FConcertCloudFileSharingService();

	/** Make a clone of this sharing service to be used. Ownership is transferred to the caller. Callers can choose to wrap this into a unique or shared pointer. */
	UE_API virtual IConcertFileSharingService* Clone() const override;

	/** Is the file sharing service on / off. */
	UE_API virtual bool IsEnabled() const override;

	/** Turn on/off the file sharing service. */
	UE_API virtual void SetEnabled(bool bInEnabled) override;

	/** Sets the desired path to use for the service.  */
	UE_API virtual void SetPath(const FString& Path) override;

	/** Get the assigned sharing path. */
	UE_API virtual FString GetPath() const override;

	/** Delete the given InFileId if it exists. Directories are supported and if a directory is supplied then DeleteDirectoryRecursively is called. */
	UE_API virtual void Delete(const FString& InFileId) override;

	/** Copies the specified file in the service storage and return a unique ID required to consume it back. */
	UE_API virtual bool Publish(const FString& Pathname, FString& OutFileId) override;

	/** Copies the content of the archive in the service storage and return a unique ID required to consume it back. */
	UE_API virtual bool Publish(FArchive& SrcAr, int64 Size, FString& OutFileId) override;

	/** Consumes a file corresponding to the unique file ID, if available. */
	UE_API virtual TSharedPtr<FArchive> CreateReader(const FString& InFileId) override;

	/** Opens a file for writing. The file id is provided as a return argument. */
	UE_API virtual TSharedPtr<FArchive> CreateWriter(FString& OutFileId) override;

private:
	/** Checks to see if the desired sharing path is valid and writable. */
	bool PathIsValid() const;
		
	bool bEnabled = false;
		
	FString SharedRootPathname;
};

#undef UE_API
