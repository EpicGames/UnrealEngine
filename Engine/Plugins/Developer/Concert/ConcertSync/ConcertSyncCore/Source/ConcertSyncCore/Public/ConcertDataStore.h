// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertDataStoreMessages.h"

#define UE_API CONCERTSYNCCORE_API

struct FConcertSessionSerializedPayload;

/**
 * Maintains a type-safe key/value map where the values are USTRUCT() structs
 * serialized into a FConcertDataStore_StoreValue. Each key/value pair has a version,
 * starting at 1 which incremented every times the value change.
 *
 * @remarks The implementation is not thread safe. It is left to the user to
 *          synchronize access to the store.
 *
 * <b>How to set up a value in the store.</b>
 * @code
 * template<typename T>
 * void MyClass::Store(const FName& Key, const T& Value)
 * {
 *     // Wraps T into its corresponding USTRUCT() if this is not already a USTRUCT()
 *     const typename TConcertDataStoreType<T>::StructType& StructWrappedValue = TConcertDataStoreType<T>::AsStructType(Value);
 *
 *     // Serialize the value.
 *     FConcertSessionSerializedPayload SerializedValue;
 *     SerializedValue.SetPayload(TConcertDataStoreType<T>::StructType::StaticStruct(), &StructWrappedValue);
 *
 *     // Store the value at version 1.
 *     DataStore.Store(Key, TConcertDataStoreType::GetFName(), SerializedValue);
 * }
 * @endcode
 *
 * <b>How to read a value from the store.</b>
 * @code
 * template<typename T>
 * TOptional<T> MyClass::Fetch(const FName& Key)
 * {
 *     FConcertDataStoreResult Result = DataStore.Fetch(Key, TConcertDataStoreType<T>::GetFName());
 *     if (Result.Code == EConcertDataStoreResultCode::Fetched)
 *     {
 *         return Result.Value->DeserializeUnchecked<T>();
 *     }
 *     return TOptional<T>();
 * }
 * @endcode
 */
class FConcertDataStore
{
public:
	/** Defines how the store updates an existing value. */
	enum class EUpdatePolicy : int
	{
		/** The existing value pointed by the shared pointer is overwritten, instantaneously updating all returned values still held by the client. */
		Overwrite,
		/** The existing shared pointer is replaced with a new shared pointer, leaving values returned to the client (shared pointers) unchanged. */
		Replace,
	};

	/**
	 * Construct the store.
	 * @param InUpdatePolicy The store update policy.
	 */
	FConcertDataStore(EUpdatePolicy InUpdatePolicy = EUpdatePolicy::Replace)
		: UpdatePolicy(InUpdatePolicy)
	{
	}

	/**
	 * Searches the store for the specified key, if not found, adds a new key/value pair, otherwise,
	 * if the stored value type matches the initial value type, fetches the stored value. The store
	 * always sets the value version to 1 when the value is added.
	 *
	 * @param Key The value name to fetch or add.
	 * @param TypeName The value type name as returned by TConcertDataStoreType::GetFName().
	 * @param SerializedValue The value to store if the key doesn't already exist.
	 * @return The operation result. The possible result codes are:
	 *     - EConcertDataStoreResultCode::Added if the key/value was added. The result holds a pointer on the newly added value.
	 *     - EConcertDataStoreResultCode::Fetched if the key was already taken and type matched. The result holds a pointer on the fetched value.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the key was already taken but the types did not match. The result holds a null pointer.
	 */
	UE_API FConcertDataStoreResult FetchOrAdd(const FName& Key, const FName& TypeName, const FConcertSessionSerializedPayload& SerializedValue);

	/**
	 * Looks up the specified key, if it doesn't exist yet, adds new key/value pair, else, if the stored value type matches
	 * the specified value type, updates the value.
	 *
	 * @param Key The value name to store.
	 * @param TypeName The value type name as returned by TConcertDataStoreType::GetFName().
	 * @param SerializedValue The value to store.
	 * @param Version If set, store the value with the specified version, otherwise, the server will set the version to 1 if the value is inserted or increment it by one if updated.
	 * @return The operation result. The possible result codes are:
	 *     - EConcertDataStoreResultCode::Added if the key/value was added. The result holds a pointer on the newly added value.
	 *     - EConcertDataStoreResultCode::Exchanged if the existing key value was updated. The result holds a pointer on the latest stored value.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the key was already taken but the value types did not match. The result holds a null pointer.
	 * @see[The example in the class documentation.](@ref FConcertDataStore)
	 */
	UE_API FConcertDataStoreResult Store(const FName& Key, const FName& TypeName, const FConcertSessionSerializedPayload& SerializedValue, const TOptional<uint32>& Version = TOptional<uint32>());

	/**
	 * Looks up the specified key, if found and types match, fetches the corresponding value. If the key is not found
	 * or the requested type doesn't match the stored type, the operation fails.
	 *
	 * @param Key The value name to lookup.
	 * @param TypeName The value type name as returned by TConcertDataStoreType::GetFName().
	 * @return The operation result. The possible result codes are:
	 *     - EConcertDataStoreResultCode::Fetched if the key value was retrieved. The result holds a pointer on the fetched value.
	 *     - EConcertDataStoreResultCode::NotFound if the key could not be found. The result holds a null pointer.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the key was found, but the requested type did not match the stored type. The result holds a null pointer.
	 * @see[The example in the class documentation.](@ref FConcertDataStore)
	 */
	UE_API FConcertDataStoreResult Fetch(const FName& Key, const FName& TypeName) const;

	/**
	 * Visits all the key/value currently stored.
	 * @param Visitor The function invoked for each key/value pair.
	 */
	UE_API void Visit(const TFunctionRef<void(const FName&, const FConcertDataStore_StoreValue&)>& Visitor) const;

	/** Returns the version of the specified key if found. */
	UE_API TOptional<uint32> GetVersion(const FName& Key) const;

	/** Returns the number of key/value pairs currently stored. */
	int32 GetSize() const { return KeyValueMap.Num(); }

private:
	UE_API FConcertDataStoreResult InternalAdd(const FName& Key, const FName& TypeName, const FConcertSessionSerializedPayload& Value, uint32 Version);

	/** Defines how the values are updated in the store. */
	EUpdatePolicy UpdatePolicy;

	/** Maps keyName/keyValue. */
	TMap<FName, FConcertDataStoreValueRef> KeyValueMap;
};

#undef UE_API
