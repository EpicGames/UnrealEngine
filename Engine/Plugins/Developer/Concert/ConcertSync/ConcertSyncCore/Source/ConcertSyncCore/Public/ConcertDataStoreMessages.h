// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "UObject/StructOnScope.h"
#include "ConcertDataStoreMessages.generated.h"

/** Response codes for Concert data store operations. */
UENUM()
enum class EConcertDataStoreResultCode : uint8
{
	/** The key/value pair was added. */
	Added,
	/** The specified key value was fetched. */
	Fetched,
	/** The specified key value was exchanged. */
	Exchanged,
	/** Reading or writing a key value in the data store failed because the specified key could not be found. */
	NotFound,
	/** Reading or writing a key value in the data store failed because the specified value type did not match the stored value type. */
	TypeMismatch,
	/** An unexpected error occurred. */
	UnexpectedError,
};

/**
 * A USTRUCT() wrapper struct, used by the implementation, to serialize/deserialize integers (of any type)
 * or bool values passed to the Concert data store API.
 */
USTRUCT()
struct FConcertDataStore_Integer
{
	GENERATED_BODY()

	/** The stored value. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Value")
	uint64 Value = 0;

	/** Converts the struct to a uint8. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator uint8() const { return static_cast<uint8>(Value); }

	/** Converts the struct to a int8. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator int8() const { return static_cast<int8>(Value); }

	/** Converts the struct to a uint16. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator uint16() const { return static_cast<uint16>(Value); }

	/** Converts the struct to a int16. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator int16() const { return static_cast<int16>(Value); }

	/** Converts the struct to a uint32. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator uint32() const { return static_cast<uint32>(Value); }

	/** Converts the struct to a int32. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator int32() const { return static_cast<int32>(Value); }

	/** Converts the struct to a uint64. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator uint64() const { return Value; }

	/** Converts the struct to a int64. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator int64() const { return static_cast<int64>(Value); }

	/** Converts the struct to a boolean. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator bool() const { return Value != 0; }
};

/**
 * A USTRUCT() wrapper struct, used by the implementation, to serialize/deserialize floating point values
 * passed to the Concert data store API.
 */
USTRUCT()
struct FConcertDataStore_Double
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Value")
	double Value = 0.0;

	/** Converts the struct to a double. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator double() const { return Value; }

	/** Converts the struct to a double. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator float() const { return static_cast<float>(Value); }
};

/**
 * A USTRUCT() wrapper struct, used by the implementation, to serialize/deserialize FName and
 * FString passed to the Concert data store API.
 */
USTRUCT()
struct FConcertDataStore_String
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Value")
	FString Value;

	/** Converts the struct to a FString. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator FString() const { return Value; }

	/** Converts the struct to a FName. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator FName() const { return FName(*Value); }
};

/**
 * A USTRUCT() wrapper struct, used by the implementation, to serialize/deserialize FText passed to
 * the Concert data store API.
 */
USTRUCT()
struct FConcertDataStore_Text
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Value")
	FText Value;

	/** Converts the struct to a FText. @see FConcertDataStore_StoreValue::DeserializeUnchecked<T>(). */
	explicit operator FText() const { return Value; }
};

/**
 * Maps a type to its corresponding USTRUCT() type. This enables deducing the
 * USTRUCT() struct when calling the Concert data store API. The non-specialized
 * version is expected to match all USTRUCT() structs "as-is" while the specialized
 * versions are meant to map basic types such as int, bool or float to their
 * corresponding USTRUCT() wrapper.
 */
template<typename T> struct TConcertDataStoreType
{
	/** The USTRUCT() struct corresponding to T. */
	typedef T StructType;

	/** Wraps a value into its corresponding USTRUCT() type. */
	static const StructType& AsStructType(const StructType& Value) { return Value; } // This is a 'pass through', no copy, no conversion.

	/** Returns the data type name. */
	static FName GetFName() { return StructType::StaticStruct()->GetFName(); }
};

/** Maps a uint8 to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<uint8>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(uint8 Value) { return StructType{Value}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_uint8")); } // Append _uint8 to distinguish from other types using FConcertDataStore_Integer.
};

/** Maps a int8 to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<int8>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(int8 Value) { return StructType{static_cast<uint64>(Value)}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_int8")); } // Append _int8 to distinguish from other types using FConcertDataStore_Integer
};

/** Maps a uint16 to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<uint16>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(uint16 Value) { return StructType{Value}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_uint16")); } // Append _uint16 to distinguish from other types using FConcertDataStore_Integer
};

/** Maps a int16 to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<int16>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(int16 Value) { return StructType{static_cast<uint64>(Value)}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_int16")); } // Append _int16 to distinguish from other types using FConcertDataStore_Integer
};

/** Maps a uint32 to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<uint32>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(uint32 Value) { return StructType{Value}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_uint32")); } // Append _uint32 to distinguish from other types using FConcertDataStore_Integer
};

/** Maps a int32 to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<int32>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(int32 Value) { return StructType{static_cast<uint64>(Value)}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_int32")); } // Append _int32 to distinguish from other types using FConcertDataStore_Integer
};

/** Maps a uint64 to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<uint64>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(uint64 Value) { return StructType{Value}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_uint64")); } // Append _int64 to distinguish from other types using FConcertDataStore_Integer
};

/** Maps a int64 to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<int64>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(int64 Value) { return StructType{static_cast<uint64>(Value)}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_int64")); } // Append _int64 to distinguish from other types using FConcertDataStore_Integer
};

/** Maps a float to its FConcertDataStore_Double USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<float>
{
	typedef FConcertDataStore_Double StructType;
	static StructType AsStructType(float Value) { return StructType{Value}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Double_float")); } // Append _float to distinguish from other types using FConcertDataStore_Double
};

/** Maps a double to its FConcertDataStore_Double USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<double>
{
	typedef FConcertDataStore_Double StructType;
	static StructType AsStructType(double Value) { return StructType{Value}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Double_double")); } // Append _double to distinguish from other types using FConcertDataStore_Double
};

/** Maps a bool to its FConcertDataStore_Integer USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<bool>
{
	typedef FConcertDataStore_Integer StructType;
	static StructType AsStructType(bool Value) { return StructType{Value != 0}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_Integer_bool")); } // Append _bool to distinguish from other types using FConcertDataStore_Integer
};

/** Maps a FName to its FConcertDataStore_String USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<FName>
{
	typedef FConcertDataStore_String StructType;
	static StructType AsStructType(const FName& Value) { return StructType{Value.ToString()}; }
	static FName GetFName() { return FName(TEXT("FConcertDataStore_String_FName")); } // Append _FName to distinguish from 'FString'
};

/** Maps a FString to its FConcertDataStore_String USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<FString>
{
	typedef FConcertDataStore_String StructType;
	static StructType AsStructType(const FString& Value) { return StructType{Value}; }
	static FName GetFName() { return StructType::StaticStruct()->GetFName(); }
};

/** Maps a FText to its FConcertDataStore_Text USTRUCT() wrapper. */
template<> struct TConcertDataStoreType<FText>
{
	typedef FConcertDataStore_Text StructType;
	static StructType AsStructType(const FText& Value) { return StructType{Value}; }
	static FName GetFName() { return StructType::StaticStruct()->GetFName(); }
};


/** A value and its meta-data as stored by a Concert data store and transported between a client and a server. */
USTRUCT()
struct FConcertDataStore_StoreValue
{
	GENERATED_BODY()

	/**
	 * The data type name as returned by TConcertDataStoreType::GetFName().
	 */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Value")
	FName TypeName;

	/**
	 * The value version number set by the server. Starting at 1 when a key/value pair
	 * is inserted, incremented by one every time it is exchanged. As an optimization
	 * in the implementation, the client may substitute, when possible, the expected
	 * value by its expected version during a CompareExchange operation if the value
	 * is large.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Value")
	uint32 Version = 0;

	/** Contains the value in its serialized and compact form. @see DeserializeUnchecked(). */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Value")
	FConcertSessionSerializedPayload SerializedValue;

	/** Deserialize @ref FConcertDataStore_StoreValue.SerializedValue field into type T, without type checking. */
	template<typename T>
	T DeserializeUnchecked() const
	{
		FStructOnScope Payload;
		SerializedValue.GetPayload(Payload);

		// When T is a USTRUCT(), for example FPoint, the operation is a pass through: static_cast<FPoint>(*reinterpert_cast<const FPoint*>(Payload.GetStructMemory()));
		// When T is a basic type, for example bool, the operation deduce its USTRUCT wrapper FConcertDataStore_Integer, then call its cast bool operator() on it to convert the int64 into bool.
		// If a USTRUCT() struct cannot be deduce from T, the code will fail to compile.
		return static_cast<T>(*reinterpret_cast<const typename TConcertDataStoreType<T>::StructType*>(Payload.GetStructMemory()));
	}
};

/** Thread safe shared reference to a FConcertDataStore_StoreValue. */
typedef TSharedRef<FConcertDataStore_StoreValue, ESPMode::ThreadSafe> FConcertDataStoreValueRef;
/** Thread safe shared pointer to a FConcertDataStore_StoreValue. */
typedef TSharedPtr<FConcertDataStore_StoreValue, ESPMode::ThreadSafe> FConcertDataStoreValuePtr;
/** Thread safe shared reference to a const FConcertDataStore_StoreValue. */
typedef TSharedRef<const FConcertDataStore_StoreValue, ESPMode::ThreadSafe> FConcertDataStoreValueConstRef;
/** Thread safe shared pointer to a const FConcertDataStore_StoreValue. */
typedef TSharedPtr<const FConcertDataStore_StoreValue, ESPMode::ThreadSafe> FConcertDataStoreValueConstPtr;

/**
 * Contains a key and its value, used by the client/server cache replication mechanism.
 * @see FConcertDataStore_ReplicateEvent
 */
USTRUCT()
struct FConcertDataStore_KeyValuePair
{
	GENERATED_BODY()

	/** The property name. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	FName Key;

	/** The property value. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	FConcertDataStore_StoreValue Value;
};

/**
 * The event message sent by the server to the client to perform the initial replication, sending
 * all currently stored key/value pairs to a new session client(s) or to notify any further changes,
 * pushing an updated key/value pair to all clients except the one who performed the change.
 */
USTRUCT()
struct FConcertDataStore_ReplicateEvent
{
	GENERATED_BODY()

	/** The initial values or the values that recently changed. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	TArray<FConcertDataStore_KeyValuePair> Values;
};

/**
 * The request used as the base class for fetch or add and compare exchange
 * requests. The end user should not use this structure directly, but use the
 * FetchOrAdd() or CompareExchange() API instead.
 * @see IConcertClientDataStore::FetchAs() methods.
 * @see IConcertClientDataStore::FetchOrAdd() methods.
 * @see IConcertClientDataStore::CompareExchange() methods.
 */
USTRUCT()
struct FConcertDataStore_Request
{
	GENERATED_BODY()

	/** The name of the value to add/fetch/compare exchange. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	FName Key;

	/** The type name of the value USTRUCT to compare and exchange as returned by TConcertDataStoreType::GetFName(). */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	FName TypeName;
};

/**
 * The request passed from the client to the server to fetch or add a key/value pair
 * from/in the data store. The end user should not use this structure directly, but use the
 * IConcertClientDataStore::FetchOrAdd() API instead. The response type for this request is
 * FConcertDataStore_Response.
 * @see IConcertClientDataStore::FetchOrAdd() methods.
 */
USTRUCT()
struct FConcertDataStore_FetchOrAddRequest : public FConcertDataStore_Request
{
	GENERATED_BODY()

	/**
	 * The property value to add if it doesn't already exist, in its serialized form.
	 * This implies the serialization is consistent across platforms and that no padding
	 * is serialized. When this is true, the serialized data is compact, binary comparable
	 * and the data store backend doesn't need to know the content format, it can only map
	 * a name and a blob and use memcmp() to compare and exchange a value.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	FConcertSessionSerializedPayload SerializedValue;
};

/**
 * The request passed from the client to the server to compare and exchange a stored value.
 * The end user should not use this structure directly, but use the IConcertClientDataStore::CompareExchange()
 * API instead. The response type for this request is a FConcertDataStore_Response.
 * @see IConcertClientDataStore::CompareExchange() methods.
 */
USTRUCT()
struct FConcertDataStore_CompareExchangeRequest : public FConcertDataStore_Request
{
	GENERATED_BODY()

	/**
	 * The expected version of the value. If non-zero, the server uses this
	 * fields to identify the expected value and ignore the 'Expected' field.
	 * This is an optimization implemented in the client/server protocol. If the
	 * expected payload is large and correspond to the value currently cached
	 * in the client, the client will send the expected version rather than
	 * the expected value to save bandwidth. The server will compare version
	 * and if they match, will perform the exchange.
	 **/
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	uint32 ExpectedVersion = 0;

	/**
	 * The expected value if 'ExpectedVersion' is zero. The field is ignored if
	 * 'ExpectedVersion' is not zero and should be left empty.
	 * @see FConcertDataStore_FetchOrAddRequest for more explanation about the format.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	FConcertSessionSerializedPayload Expected;

	/** The desired value to store. @see FConcertDataStore_FetchOrAdd request for more explanation about the format. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	FConcertSessionSerializedPayload Desired;
};

/** The response to a FConcertDataStore_FetchOrAddRequest or FConcertDataStore_CompareExchangeRequest requests, sent from the server to the client. */
USTRUCT()
struct FConcertDataStore_Response
{
	GENERATED_BODY()

	/**
	 * The result code to the request. Possible values depends on the request.
	 * @see IConcertClientDataStore::FetchOrAdd()
	 * @see IConcertClientDataStore::FetchAs()
	 * @see IConcertClientDataStore::CompareExchange().
	 */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	EConcertDataStoreResultCode ResultCode = EConcertDataStoreResultCode::UnexpectedError;

	/**
	 * Contains the stored value if the ResponseCode == EConcertDataStoreResultCode::Fetched,
	 * otherwise, it is left empty. The server doesn't sent back the stored value when the
	 * client successfully stores it. The client is expected to keep the value it sent.
	 * @see TConcertDataStoreResult::GetValue() to deserialize the payload.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Concert Data Store Message")
	FConcertDataStore_StoreValue Value;
};

/**
 * Contains the result of a Concert data store operation. This class, as opposed to
 * FConcertDataStore_Response, contains a shared pointer on the value to manage the
 * lifetime of a value in the multi-versions client cache.
 */
class FConcertDataStoreResult
{
public:
	/** Constructs a result containing the EConcertDataStoreResultCode::UnexpectedError code and no value. */
	FConcertDataStoreResult() : Code(EConcertDataStoreResultCode::UnexpectedError) {}

	/** Construct a result containing the specified error code and no value.*/
	FConcertDataStoreResult(EConcertDataStoreResultCode InErrorCode) : Code(InErrorCode) {}

	/** Construct a result containing the specified code (added, fetched, exchanged) and its corresponding value. */
	FConcertDataStoreResult(EConcertDataStoreResultCode InCode, FConcertDataStoreValueConstPtr InValue) : Code(InCode), Value(MoveTemp(InValue)) {}

	/** The operation result code. */
	EConcertDataStoreResultCode Code;

	/** The value returned to the caller unless an error occurred. */
	FConcertDataStoreValueConstPtr Value;
};

/**
 * Wraps the weakly typed result of a Concert data store operation into a
 * strongly typed result. Type checking occurs during the transaction with
 * the store and as long as the result is valid, the stored value can be
 * read safely.
 *
 * @tparam T - Must be a USTRUCT() struct or one of the basic type from which a
 *             USTRUCT() struct can be deduced like int, float, bool, ...
 *
 * @see TConcertDataStoreType.
 */
template<typename T>
class TConcertDataStoreResult
{
public:
	/** Constructs a result containing the EConcertDataStoreResultCode::UnexpectedError code and no value. */
	TConcertDataStoreResult() { }

	/** Constructs a strongly typed result from a weakly typed one. */
	TConcertDataStoreResult(const FConcertDataStoreResult& InResult) : Result(InResult) {}

	/** Constructs a strongly typed result from a weakly typed one. */
	TConcertDataStoreResult(FConcertDataStoreResult&& InResult) : Result(Forward<FConcertDataStoreResult>(InResult)) {}

	/** Returns true if the result hold a value that the value can be read. */
	operator bool() const { return Result.Value.IsValid(); }

	/** Returns the data store result code for the operation. */
	EConcertDataStoreResultCode GetCode() const { return Result.Code; }

	/**
	 * Returns true if the underlying value is valid. The value is not
	 * valid if the result code is EConcertDataStoreResultCode::TypeMismatch
	 * or EConcertDataStoreResultCode::NotFound.
	 */
	bool IsValid() const { return Result.Value.IsValid(); }

	/** 
	 * Deserializes the stored value into type T. The data store performs type checking and
	 * the deserialization is type safe as long as IsValid() or operator bool() returns true.
	 */
	T GetValue() const { return Result.Value->DeserializeUnchecked<T>(); }

private:
	/** The weakly-typed operation result. */
	FConcertDataStoreResult Result;
};
