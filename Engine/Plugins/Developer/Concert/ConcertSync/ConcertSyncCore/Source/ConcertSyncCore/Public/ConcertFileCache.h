// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/LruCache.h"
#include "Misc/DateTime.h"
#include "Templates/SharedPointer.h"

#define UE_API CONCERTSYNCCORE_API

/**
 * Maintains an in-memory cache of file data, attempting to keep the cache within the given upper memory threshold.
 * @note Will automatically re-cache files if they are changed on disk (via a timestamp change), and un-cache files that are deleted from disk.
 */
class FConcertFileCache
{
public:
	UE_API FConcertFileCache(const int32 InMinimumNumberOfFilesToCache, const uint64 InMaximumNumberOfBytesToCache);
	UE_API ~FConcertFileCache();

	/**
	 * Non-copyable.
	 */
	FConcertFileCache(const FConcertFileCache&) = delete;
	FConcertFileCache& operator=(const FConcertFileCache&) = delete;

	/**
	 * Cache the given file (if valid).
	 * @note This function ignores the current cached file state, so will re-cache the file even if it has already been cached.
	 * @return True if the file was cached, false otherwise.
	 */
	UE_API bool CacheFile(const FString& InFilename);

	/**
	 * Save the given file and add it to the cache.
	 * @return True if the file was cached, false otherwise.
	 */
	UE_API bool SaveAndCacheFile(const FString& InFilename, TArray<uint8>&& InFileData);

	/**
	 * Uncache the given file.
	 */
	UE_API void UncacheFile(const FString& InFilename);

	/**
	 * Find or cache the given file, and get its data.
	 * @return True if the file was found or cached, false otherwise.
	 */
	UE_API bool FindOrCacheFile(const FString& InFilename, TArray<uint8>& OutFileData);

	/**
	 * Find the given file, and get its data.
	 * @return True if the file was found, false otherwise.
	 */
	UE_API bool FindFile(const FString& InFilename, TArray<uint8>& OutFileData) const;

	/**
	 * Is the given file cached?
	 * @return True if the file is cached, false otherwise.
	 */
	UE_API bool HasCachedFile(const FString& InFilename) const;

private:
	/**
	 * Trim the cache to attempt to keep it within the requested upper memory threshold.
	 */
	UE_API void TrimCache();

	/** Internal cache entry that scopes counting the cached file size into the outer file-cache */
	class FInternalCacheEntry
	{
	public:
		FInternalCacheEntry(TArray<uint8>&& InFileData, const FDateTime& InFileTimestamp, uint64& InTotalCachedFileDataBytesRef)
			: FileData(MoveTemp(InFileData))
			, FileTimestamp(InFileTimestamp)
			, TotalCachedFileDataBytesRef(InTotalCachedFileDataBytesRef)
		{
			TotalCachedFileDataBytesRef += FileData.Num();
		}

		~FInternalCacheEntry()
		{
			TotalCachedFileDataBytesRef -= FileData.Num();
		}

		FInternalCacheEntry(const FInternalCacheEntry&) = delete;
		FInternalCacheEntry& operator=(const FInternalCacheEntry&) = delete;

		const TArray<uint8>& GetFileData() const
		{
			return FileData;
		}

		const FDateTime& GetFileTimestamp() const
		{
			return FileTimestamp;
		}

		void SetFile(TArray<uint8>&& InFileData, const FDateTime& InFileTimestamp)
		{
			TotalCachedFileDataBytesRef -= FileData.Num();
			
			FileData = MoveTemp(InFileData);
			FileTimestamp = InFileTimestamp;

			TotalCachedFileDataBytesRef += FileData.Num();
		}

	private:
		TArray<uint8> FileData;
		FDateTime FileTimestamp;
		uint64& TotalCachedFileDataBytesRef;
	};

	const int32 MinimumNumberOfFilesToCache;
	const uint64 MaximumNumberOfBytesToCache;

	uint64 TotalCachedFileDataBytes;
	TLruCache<FString, TSharedPtr<FInternalCacheEntry>> InternalCache;
};

#undef UE_API
