// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertFileSharingService.h"

#define UE_API CONCERTSYNCCORE_API

/**
 * Share files using a local directory. It works only for one client and one server running on the same machine. The publisher of
 * a file stores a file for sharing and gets a unique ID to consume the file back. It is the publisher responsibility to share the
 * ID with the consumer (i.e. through a concert msg). The shared files are automatically deleted once consumed or when the file
 * expires.
 * 
 * @note This is mainly designed for the recovery system.
 */
class FConcertLocalFileSharingService : public IConcertFileSharingService
{
public:
	/**
	 * Construct the local file sharing service.
	 * @param Role Appended to the shared directory (like DisasterRecovery) as a hint for the temp folder purpose. Client and server must use the same value.
	 * @note The service uses the Engine intermediate directory to share the files. The files are automatically deleted after consumption. In case of crash
	 *       the temporary shared files are deleted on next reboot if no other client/server on the machine are not running.
	 */
	UE_API FConcertLocalFileSharingService(const FString& Role);
	UE_API virtual ~FConcertLocalFileSharingService();

	/** Make a clone of this sharing service to be used. Ownership is transferred to the caller. Callers can choose to wrap this into a unique or shared pointer. */
	UE_API virtual IConcertFileSharingService* Clone() const override;

	/** Indicates if this file sharing service is enabled. Local file sharing is always enabled and it is not possible to disable at runtime. */
	UE_API virtual bool IsEnabled() const override;

	/** This is a no-op and disabling the local file sharing service is not supported.  */
	UE_API virtual void SetEnabled(bool bInEnabled) override;

	/** Sets the desired path to use for the service.  Implementors have the right to ignore the path specified.  This class does. */
	UE_API virtual void SetPath(const FString& Path) override;

	/** Get the assigned sharing path. */
	UE_API virtual FString GetPath() const override;

	/** Delete is not supported for local file sharing and only provided to satisfy interfac requirement. */
	UE_API virtual void Delete(const FString& InFileId) override;

	/** Copies the specified file in the service storage and return a unique ID required to consume it back. The stored file expires in 1 hour. */
	UE_API virtual bool Publish(const FString& Pathname, FString& OutFileId) override;

	/** Copies the content of the archive in the service storage and return a unique ID required to consume it back. The stored file expires in 1 hour. */
	UE_API virtual bool Publish(FArchive& SrcAr, int64 Size, FString& OutFileId) override;

	/** Consumes a file corresponding to the unique file ID, if available. The file is deleted from the service storage when the shared pointer goes out of scope. */
	UE_API virtual TSharedPtr<FArchive> CreateReader(const FString& InFileId) override;

	/** Opens a file for writing. The file id is provided as a return argument. */
	UE_API virtual TSharedPtr<FArchive> CreateWriter(FString& OutFileId) override;
private:
	/** Load the list of concurrent file sharing service running across all processes. */
	UE_API void LoadActiveServices(TArray<uint32>& OutPids);

	/** Saved the list of concurrent file sharing service running across all processes. */
	UE_API void SaveActiveServices(const TArray<uint32>& InPids);

	/** Track active services instances across all processes and clean up any left over files that expired if no services are running anymore. */
	UE_API void RemoveDeadProcessesAndFiles(TArray<uint32>& InOutPids);

private:
	FString SharedRootPathname;
	FString SystemMutexName;
	FString ActiveServicesRepositoryPathname;
};

#undef UE_API
