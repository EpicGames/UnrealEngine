// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Guid.h"

struct FConcertPackageInfo;

namespace UE::ConcertSyncCore::ConcertPackageEvents
{
	struct FConcertBaseSendPackageParams
	{
		/** Identifies begin and finish send events so you can map them to each other. */
		FGuid TransmissionId;
		
		/** General information about the package */
		const FConcertPackageInfo& PackageInfo;
		
		/**
		 * The remote endpoint that is interacting with this package.
		 *
		 * If we're the server, then there are 2 options:
		 * 1. This is a client that's sending us data
		 * 2. This is a client that we're sending the data to
		 */
		FGuid RemoteEndpointId;
	};
	
	struct FConcertBeginSendPackageParams : FConcertBaseSendPackageParams
	{
		/** The size of the package data to be received */
		uint64 PackageNumBytes;
	};
	
	struct FConcertFinishSendPackageParams : FConcertBaseSendPackageParams
	{
		/**
		 * The ID of the message that causes the package to be acknowledged.
		 *
		 * In the case of sending to a client, it is the acknowledgement message.
		 * In the case of receiving from client, it is the message transmitting the sync activity.
		 */
		FGuid MessageId;
	};

	struct FConcertRejectSendPackageParams : FConcertFinishSendPackageParams
	{};
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FConcertBeginSendPackageDelegate, const FConcertBeginSendPackageParams&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FConcertFinishSendPackageDelegate, const FConcertFinishSendPackageParams&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FConcertRejectSendPackageDelegate, const FConcertRejectSendPackageParams&);
	
	// Note: These events are only called when package contents is sent. Package activities do not always contain package data in which case these events are not called.
	// Caution: For now these events are only called on the server application; they're not called in client applications.
	
	/** Called when we begin sending a package to a remote endpoint. RemoteEndpointId is to be interpreted as the receiver. */
	CONCERTSYNCCORE_API FConcertBeginSendPackageDelegate& OnLocalBeginSendPackage();
	/** Called when we finish sending a package to a remote endpoint. They've sent us an ACK. RemoteEndpointId is to be interpreted as the receiver. */
	CONCERTSYNCCORE_API FConcertFinishSendPackageDelegate& OnLocalFinishSendPackage();

	/** Called when we begin receiving a package from a remote endpoint. They've let us know in advance via an event. RemoteEndpointId is to be interpreted as the sender. */
	CONCERTSYNCCORE_API FConcertBeginSendPackageDelegate& OnRemoteBeginSendPackage();
	/** Called when we finished receiving a package from a remote endpoint. We've processed the data locally. RemoteEndpointId is to be interpreted as the sender. */
	CONCERTSYNCCORE_API FConcertFinishSendPackageDelegate& OnRemoteFinishSendPackage();
	/** Called when we finished receiving a package from a remote endpoint but decided to reject the data. Called only on the server. RemoteEndpointId is to be interpreted as the sender. */
	CONCERTSYNCCORE_API FConcertRejectSendPackageDelegate& OnRejectRemoteSendPackage();
}
