// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertPresenceEvents.generated.h"

USTRUCT()
struct FConcertClientPresenceEventBase
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 TransactionUpdateIndex = 0;
};

USTRUCT()
struct FConcertClientPresenceVisibilityUpdateEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ModifiedEndpointId;

	UPROPERTY()
	bool bVisibility = true;
};

USTRUCT()
struct FConcertClientPresenceInVREvent
{
	GENERATED_BODY()

	UPROPERTY()
	FName VRDevice;
};

UENUM()
enum class EPresenceModeType
{
	Desktop,
	VRMode
};

USTRUCT()
struct FConcertClientPresenceDataUpdateEvent : public FConcertClientPresenceEventBase
{
	GENERATED_BODY()

	FConcertClientPresenceDataUpdateEvent()
		: WorldPath()
		, Position(FVector::ZeroVector)
		, Orientation(FQuat::Identity)
		, PresenceType(EPresenceModeType::Desktop)
	{
	}

	/** The non-PIE/SIE world path. In PIE/SIE, the world context and its path is different than the non-PIE/SIE context.
	    For presence management, we are interested in the Non-PIE/SIE world path and this is what is emitted even if the user is in PIE/SIE. */
	UPROPERTY()
	FName WorldPath;

	UPROPERTY()
	FVector Position;

	UPROPERTY()
	FQuat Orientation;

	UPROPERTY()
	EPresenceModeType PresenceType;
};

USTRUCT()
struct FConcertClientDesktopPresenceUpdateEvent : public FConcertClientPresenceEventBase
{
	GENERATED_BODY()

	FConcertClientDesktopPresenceUpdateEvent()
		: TraceStart(FVector::ZeroVector)
		, TraceEnd(FVector::ZeroVector)
		, bMovingCamera(false)
	{
	}

	UPROPERTY()
	FVector TraceStart;

	UPROPERTY()
	FVector TraceEnd;

	UPROPERTY()
	bool bMovingCamera;
};

USTRUCT()
struct FConcertLaserData
{
	GENERATED_BODY()

	FConcertLaserData()
		: LaserStart(FVector::ZeroVector)
		, LaserEnd(FVector::ZeroVector)
	{}

	FConcertLaserData(FVector Start, FVector End)
		: LaserStart(MoveTemp(Start))
		, LaserEnd(MoveTemp(End))
	{}

	bool IsValid() const
	{
		return LaserStart != FVector::ZeroVector || LaserEnd != FVector::ZeroVector;
	}

	UPROPERTY()
	FVector LaserStart;

	UPROPERTY()
	FVector LaserEnd;
};

USTRUCT()
struct FConcertClientVRPresenceUpdateEvent : public FConcertClientPresenceEventBase
{
	GENERATED_BODY()

	FConcertClientVRPresenceUpdateEvent()
		: LeftMotionControllerPosition(FVector::ZeroVector)
		, LeftMotionControllerOrientation(FQuat::Identity)
		, RightMotionControllerPosition(FVector::ZeroVector)
		, RightMotionControllerOrientation(FQuat::Identity)
	{}

	UPROPERTY()
	FVector LeftMotionControllerPosition;

	UPROPERTY()
	FQuat LeftMotionControllerOrientation;

	UPROPERTY()
	FVector RightMotionControllerPosition;

	UPROPERTY()
	FQuat RightMotionControllerOrientation;

	UPROPERTY()
	FConcertLaserData Lasers[2];
};
