// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/QualifiedFrameTime.h"
#include "ConcertMessageData.h"
#include "ConcertSequencerMessages.generated.h"


/**
 * Enum for the current Sequencer player status, should match EMovieScenePlayerStatus::Type
 * Defined here to not have a dependency on the MovieScene module.
 */
UENUM()
enum class EConcertMovieScenePlayerStatus : uint8
{
	Stopped,
	Playing,
	Scrubbing,
	Jumping,
	Stepping,
	Paused,
	MAX
};

USTRUCT()
struct FConcertSequencerState
{
	GENERATED_BODY()

	/** The full path name to the root sequence that is open on the sequencer */
	UPROPERTY()
	FString SequenceObjectPath;

	/** The time that the sequence is at */
	UPROPERTY()
	FQualifiedFrameTime Time;

	/** The current status of the sequencer player */
	UPROPERTY()
	EConcertMovieScenePlayerStatus PlayerStatus;

	UPROPERTY()
	FFrameNumberRange PlaybackRange;

	/** The current playback speed */
	UPROPERTY()
	float PlaybackSpeed;

	/** Indicate if we are currently looping */
	UPROPERTY()
	bool bLoopMode = false;

	/**
	 * In the case that the SequenceObjectPath points to a take preset, we capture the preset data
	 * into a payload that can be applied to take that we are going to open. We store it in the state
	 * so that we can play it back when new users join.
	 */
	UPROPERTY()
	FConcertByteArray TakeData;

	FConcertSequencerState()
		: PlayerStatus(EConcertMovieScenePlayerStatus::Stopped)
		, PlaybackSpeed(1.0f)
	{}
};

/**
 * Event that signals a Sequencer just been opened.
 */
USTRUCT()
struct FConcertSequencerOpenEvent
{
	GENERATED_BODY()

	/** The full path name to the root sequence of the sequencer that just opened. */
	UPROPERTY()
	FString SequenceObjectPath;

	/**
	 * In the case that the SequenceObjectPath points to a take preset, we capture the preset data
	 * into a payload that can be applied to take that we are going to open. We store it in the state
	 * so that we can play it back when new users join.
	 */
	UPROPERTY()
	FConcertByteArray TakeData;
};

/**
 * Event that signals a Sequencer just been closed.
 */
USTRUCT()
struct FConcertSequencerCloseEvent
{
	GENERATED_BODY()

	/** The full path name to the root sequence of the sequencer that just closed. */
	UPROPERTY()
	FString SequenceObjectPath;

	UPROPERTY()
	bool bControllerClose = false;

	UPROPERTY()
	int32 EditorsWithSequencerOpened = -1;
};

/**
 * Event that signals a sequencer UI has changed the current state
 */
USTRUCT()
struct FConcertSequencerStateEvent
{
	GENERATED_BODY()

	/** The new state that the sequence is at */
	UPROPERTY()
	FConcertSequencerState State;
};

/**
 * Event that represent the current open sequencer states to a newly connected client
 */
USTRUCT()
struct FConcertSequencerStateSyncEvent
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FConcertSequencerState> SequencerStates;
};

/**
 * An event that represents a time changes on the sequencer. This can happen via take recorder
 * which will shift the active take _if_ Start At Timecode is enabled.
 */
USTRUCT()
struct FConcertSequencerTimeAdjustmentEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FFrameNumber PlaybackStartFrame;

	UPROPERTY()
	FString SequenceObjectPath;
};


/**
 * Event indicating one or more sequences have been added or removed from the
 * set of sequences to keep preloaded for quick dynamic instantiation.
 *
 * Can be sent by clients as a request to add or remove their references.
 *
 * Can also be received from the server in response to changes to the active set,
 * or as an initial snapshot of the complete set when joining a session.
 */
USTRUCT()
struct FConcertSequencerPreloadRequest
{
	GENERATED_BODY()

	/** The list of full paths to affected sequences. */
	UPROPERTY()
	TArray<FTopLevelAssetPath> SequenceObjectPaths;

	/** True if being added to the preload set, false if being removed. */
	UPROPERTY()
	bool bShouldBePreloaded = false;
};

UENUM()
enum class EConcertSequencerPreloadStatus : uint8
{
	Pending,
	Succeeded,
	Failed,
};

/**
 * Can be sent as an event by clients to indicate loading success/failure
 * result of attempting to preload one or more sequence assets.
 */
USTRUCT()
struct FConcertSequencerPreloadAssetStatusMap
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FTopLevelAssetPath, EConcertSequencerPreloadStatus> Sequences;
};

/**
 * Sent as an event by server with preload status of one or more clients.
 */
USTRUCT()
struct FConcertSequencerPreloadClientStatusMap
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FGuid, FConcertSequencerPreloadAssetStatusMap> ClientEndpoints;

	/** Merge (add or replace) statuses for a specified client into this object. */
	void UpdateFrom(const FGuid& Client, const FConcertSequencerPreloadAssetStatusMap& Updates)
	{
		FConcertSequencerPreloadAssetStatusMap& List = ClientEndpoints.FindOrAdd(Client);
		for (const TPair<FTopLevelAssetPath, EConcertSequencerPreloadStatus>& Update : Updates.Sequences)
		{
			List.Sequences.FindOrAdd(Update.Key) = Update.Value;
		}
	}

	/** Clear all clients' references to a specified asset. */
	void Remove(const FTopLevelAssetPath& Sequence)
	{
		for (TPair<FGuid, FConcertSequencerPreloadAssetStatusMap>& Client : ClientEndpoints)
		{
			Client.Value.Sequences.Remove(Sequence);
		}
	}
};
