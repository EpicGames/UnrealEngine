// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IdentifierTable/ConcertTransportArchives.h"
#include "UObject/SoftObjectPath.h"

#define UE_API CONCERTSYNCCORE_API

struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPtr;
struct FWeakObjectPtr;

struct FConcertSessionVersionInfo;

DECLARE_DELEGATE_OneParam(FConcertSyncRemapObjectPath, FString& /*ObjectPath*/)
DECLARE_DELEGATE_RetVal_OneParam(bool, FConcertSyncObjectPathBelongsToWorld, const FStringView /*ObjectPath*/)
DECLARE_DELEGATE_OneParam(FConcertSyncEncounteredMissingObject, const FStringView /*ObjectPath*/)

namespace ConcertSyncUtil
{
CONCERTSYNCCORE_API bool CanExportProperty(const FProperty* Property, const bool InIncludeEditorOnlyData);

CONCERTSYNCCORE_API void ResetObjectPropertiesToArchetypeValues(UObject* Object, const bool InIncludeEditorOnlyData);

struct FResetValueOptions
{
	/**
	 * True if we should also reset editor-only properties.
	 */
	bool bIncludeEditorOnlyData = true;

	/**
	 * True when Object is considered a duplicate of Template (eg, two instances with the same archetype, where we want to make Object the same as Template).
	 * @note If true this sets PPF_Duplicate on the archive when copying data, and automatically skips CPF_DuplicateTransient properties.
	 */
	bool bIsDuplicate = false;

	/**
	 * True to only consider things marked CPF_SaveGame.
	 * @note If true this sets the ArIsSaveGame flag on the archive when copying the data.
	 */
	bool bSaveGameOnly = false;

	/**
	 * Set of objects that should have their references skipped when resetting values.
	 * @note Objects skipped via this set are not passed to ShouldSkipObjectReference.
	 */
	TSet<FSoftObjectPath> ObjectsToSkip;

	/**
	 * Filter used to decide whether the given property should be skipped when resetting values.
	 * @note Properties skipped internally by the archive will not be passed to this function.
	 */
	TFunction<bool(const FArchive& Ar, const FProperty* Property)> ShouldSkipProperty;

	/**
	 * Filter used to decide whether the given property reference should be skipped when resetting values.
	 * @param OldValue The current object reference from Object.
	 * @param NewValue The proposed new object reference from Template.
	 */
	TFunction<bool(const FArchive& Ar, const FSoftObjectPath& OldValue, const FSoftObjectPath& NewValue)> ShouldSkipObjectReference;
};

CONCERTSYNCCORE_API void ResetObjectPropertiesToTemplateValues(UObject* Object, const UObject* Template, const FResetValueOptions& Options);

CONCERTSYNCCORE_API const FSoftObjectPath& GetSkipObjectPath();
}

/** Util to handle remapping objects within the source world to be the equivalent objects in the destination world */
class FConcertSyncWorldRemapper
{
public:
	FConcertSyncWorldRemapper() = default;

	FConcertSyncWorldRemapper(FString InSourceWorldPathName, FString InDestWorldPathName)
		: SourceWorldPathName(MoveTemp(InSourceWorldPathName))
		, DestWorldPathName(MoveTemp(InDestWorldPathName))
	{
	}

	UE_API FString RemapObjectPathName(const FString& InObjectPathName) const;

	UE_API bool ObjectBelongsToWorld(const FString& InObjectPathName) const;

	UE_API bool HasMapping() const;

	FConcertSyncRemapObjectPath RemapDelegate;
	FConcertSyncObjectPathBelongsToWorld ObjectPathBelongsToWorldDelegate;

private:
	FString SourceWorldPathName;
	FString DestWorldPathName;
};

/** Archive for writing objects in a way that they can be sent to another instance via Concert */
class FConcertSyncObjectWriter : public FConcertIdentifierWriter
{
public:

	UE_API FConcertSyncObjectWriter(FConcertLocalIdentifierTable* InLocalIdentifierTable, UObject* InObj, TArray<uint8>& OutBytes, const bool InIncludeEditorOnlyData, const bool InSkipAssets, const FConcertSyncRemapObjectPath& InRemapDelegate);
	UE_API FConcertSyncObjectWriter(FConcertLocalIdentifierTable* InLocalIdentifierTable, UObject* InObj, TArray<uint8>& OutBytes, const bool InIncludeEditorOnlyData, const bool InSkipAssets);

	UE_API void SerializeObject(const UObject* InObject, const TArray<const FProperty*>* InPropertiesToWrite = nullptr, bool bAllowOuters = false);
	UE_API void SerializeProperty(const FProperty* InProp, const UObject* InObject);

	using FConcertIdentifierWriter::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	UE_API virtual FArchive& operator<<(UObject*& Obj) override;
	UE_API virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	UE_API virtual FArchive& operator<<(FObjectPtr& Obj) override;
	UE_API virtual FArchive& operator<<(FSoftObjectPtr& AssetPtr) override;
	UE_API virtual FArchive& operator<<(FSoftObjectPath& AssetPtr) override;
	UE_API virtual FArchive& operator<<(FWeakObjectPtr& Value) override;
	UE_API virtual FString GetArchiveName() const override;
	UE_API virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
	//~ End FArchive Interface

	void SetSerializeNestedObjects(bool bInSerializeNestedObjects)
	{
		bSerializeNestedObjects = bInSerializeNestedObjects;
	}
private:
	typedef TFunction<bool(const FProperty*)> FShouldSkipPropertyFunc;
	bool bSkipAssets;
	bool bSerializeNestedObjects = false;
	FString PackageName;
	FShouldSkipPropertyFunc ShouldSkipPropertyFunc;
	FConcertSyncRemapObjectPath RemapObjectPathDelegate;
	TSet<FString> CollectedObjects;
};

/** Archive for reading objects that have been received from another instance via Concert */
class FConcertSyncObjectReader : public FConcertIdentifierReader
{
public:
	UE_API FConcertSyncObjectReader(const FConcertLocalIdentifierTable* InLocalIdentifierTable, FConcertSyncWorldRemapper InWorldRemapper, const FConcertSessionVersionInfo* InVersionInfo, UObject* InObj, const TArray<uint8>& InBytes, const FConcertSyncEncounteredMissingObject& InEncounteredMissingObjectDelegate);
	UE_API FConcertSyncObjectReader(const FConcertLocalIdentifierTable* InLocalIdentifierTable, FConcertSyncWorldRemapper InWorldRemapper, const FConcertSessionVersionInfo* InVersionInfo, UObject* InObj, const TArray<uint8>& InBytes);

	UE_API void SerializeObject(UObject* InObject);
	UE_API void SerializeProperty(const FProperty* InProp, UObject* InObject);

	using FConcertIdentifierReader::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	UE_API virtual FArchive& operator<<(UObject*& Obj) override;
	UE_API virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	UE_API virtual FArchive& operator<<(FObjectPtr& Obj) override;
	UE_API virtual FArchive& operator<<(FSoftObjectPtr& AssetPtr) override;
	UE_API virtual FArchive& operator<<(FSoftObjectPath& AssetPtr) override;
	UE_API virtual FArchive& operator<<(FWeakObjectPtr& Value) override;
	UE_API virtual FString GetArchiveName() const override;
	//~ End FArchive Interface

	void SetSerializeNestedObjects(bool bInSerializeNestedObjects)
	{
		bSerializeNestedObjects = bInSerializeNestedObjects;
	}
protected:
	virtual void OnObjectSerialized(const FSoftObjectPath& Obj) {}

private:
	bool bSerializeNestedObjects = false;
	UObject* CurrentOuter = nullptr;
	FConcertSyncWorldRemapper WorldRemapper;
	FConcertSyncEncounteredMissingObject EncounteredMissingObjectDelegate;
};

/** Archive for rewriting identifiers (currently names) so that they belong to a different identifier table */
class FConcertSyncObjectRewriter : public FConcertIdentifierRewriter
{
public:
	UE_API FConcertSyncObjectRewriter(const FConcertLocalIdentifierTable* InLocalIdentifierTable, FConcertLocalIdentifierTable* InRewriteIdentifierTable, const FConcertSessionVersionInfo* InVersionInfo, TArray<uint8>& InBytes);

	UE_API void RewriteObject(const UClass* InClass);
	UE_API void RewriteProperty(const FProperty* InProp);

	using FConcertIdentifierRewriter::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	UE_API virtual FArchive& operator<<(UObject*& Obj) override;
	UE_API virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	UE_API virtual FArchive& operator<<(FObjectPtr& Obj) override;
	UE_API virtual FArchive& operator<<(FSoftObjectPtr& AssetPtr) override;
	UE_API virtual FArchive& operator<<(FSoftObjectPath& AssetPtr) override;
	UE_API virtual FArchive& operator<<(FWeakObjectPtr& Value) override;
	UE_API virtual FString GetArchiveName() const override;
	//~ End FArchive Interface

protected:
	/** Called to rewrite the object path that will be stored in the persistent object value (InBytes) */
	virtual void OnObjectSerialized(FSoftObjectPath& Obj) {}

	/** Called with the result of OnObjectSerialized to get the object path to set on the in-memory object value (the output of operator<<) */
	virtual FSoftObjectPath GetOutputObjectPath(const FSoftObjectPath& Obj) { return Obj; }
};

#undef UE_API
