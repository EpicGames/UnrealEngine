// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/EBreakBehavior.h"
#include "Templates/PimplPtr.h"

#define UE_API CONCERTSYNCCORE_API

class FStructOnScope;
enum class EConcertSyncActivityEventType : uint8;
enum class EConcertSyncActivityFlags : uint8;
enum class EConcertSyncReplicationActivityType : uint8;
struct FConcertPackageDataStream;
struct FConcertPackageInfo;
struct FConcertSessionFilter;
struct FConcertSyncActivity;
struct FConcertSyncConnectionActivity;
struct FConcertSyncConnectionEvent;
struct FConcertSyncEndpointData;
struct FConcertSyncEndpointIdAndData;
struct FConcertSyncLockActivity;
struct FConcertSyncLockEvent;
struct FConcertSyncPackageEventData;
struct FConcertSyncReplicationActivity;
struct FConcertSyncReplicationEvent;
struct FConcertSyncTransactionActivity;
struct FConcertSyncTransactionEvent;
struct FConcertTransactionFinalizedEvent;
struct FScriptContainerElement;
template <typename OptionalType> struct TOptional;

class FConcertFileCache;
class FConcertSyncSessionDatabaseStatements;

class FSQLiteDatabase;
enum class ESQLiteDatabaseOpenMode : uint8;

using FConsumePackageActivityFunc = TFunctionRef<void(FConcertSyncActivity&&/*BasePart*/, FConcertSyncPackageEventData& /*EventPart*/)>;
using FIteratePackageActivityFunc = TFunctionRef<EBreakBehavior(FConcertSyncActivity&&/*BasePart*/, FConcertSyncPackageEventData& /*EventPart*/)>;
using FIterateActivityFunc = TFunctionRef<EBreakBehavior(FConcertSyncActivity&&)>;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnActivityProduced, const FConcertSyncActivity&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPackageSaved, const FName& PackageName);

/**
 * Database of activities that have happened in a Concert Sync Session.
 * Stores the activity index and their associated data.
 */
class FConcertSyncSessionDatabase
{
public:
	
	UE_API FConcertSyncSessionDatabase();
	UE_API ~FConcertSyncSessionDatabase();

	FConcertSyncSessionDatabase(const FConcertSyncSessionDatabase&) = delete;
	FConcertSyncSessionDatabase& operator=(const FConcertSyncSessionDatabase&) = delete;

	UE_API FConcertSyncSessionDatabase(FConcertSyncSessionDatabase&&);
	UE_API FConcertSyncSessionDatabase& operator=(FConcertSyncSessionDatabase&&);

	/**
	 * Is this a valid database? (ie, has been successfully opened).
	 */
	UE_API bool IsValid() const;

	/**
	 * Open (or create) a database file.
	 *
	 * @param InSessionPath				The root path to store all the data for this session under.
	 *
	 * @return True if the database was opened, false otherwise.
	 */
	UE_API bool Open(const FString& InSessionPath);

	/**
	 * Open (or create) a database file.
	 *
	 * @param InSessionPath				The root path to store all the data for this session under.
	 * @param InOpenMode				How should the database be opened?
	 *
	 * @return True if the database was opened, false otherwise.
	 */
	UE_API bool Open(const FString& InSessionPath, const ESQLiteDatabaseOpenMode InOpenMode);

	/**
	 * Close an open database file.
	 *
	 * @param InDeleteDatabase			True if the session database and its associated data should be deleted after closing the database.
	 */
	UE_API bool Close(const bool InDeleteDatabase = false);

	/**
	 * Get the filename of the currently open database, or an empty string.
	 * @note The returned filename will be an absolute pathname.
	 */
	UE_API FString GetFilename() const;

	/**
	 * Get the last error reported by this database.
	 */
	UE_API FString GetLastError() const;

	/**
	 * Add a new connection activity to this database, assigning it both an activity and connection event ID.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the server to populate its version of the session database.
	 *
	 * @param InConnectionActivity		The connection activity to add (the ActivityId, EventTime, EventType, and EventId members are ignored).
	 * @param OutActivityId				Populated with the ID of this activity in the database.
	 * @param OutConnectionEventId		Populated with the ID of the connection event in the database (@see GetConnectionEvent).
	 *
	 * @return True if the connection activity was added, false otherwise.
	 */
	UE_API bool AddConnectionActivity(const FConcertSyncConnectionActivity& InConnectionActivity, int64& OutActivityId, int64& OutConnectionEventId);

	/**
	 * Add a new lock activity to this database, assigning it both an activity and lock event ID.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the server to populate its version of the session database.
	 *
	 * @param InLockActivity			The lock activity to add (the ActivityId, EventTime, EventType, and EventId members are ignored).
	 * @param OutActivityId				Populated with the ID of this activity in the database.
	 * @param OutLockEventId			Populated with the ID of the lock event in the database (@see GetLockEvent).
	 *
	 * @return True if the lock activity was added, false otherwise.
	 */
	UE_API bool AddLockActivity(const FConcertSyncLockActivity& InLockActivity, int64& OutActivityId, int64& OutLockEventId);

	/**
	 * Add a new transaction activity to this database, assigning it both an activity and transaction event ID.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the server to populate its version of the session database.
	 *
	 * @param InTransactionActivity		The transaction activity to add (the ActivityId, EventTime, EventType, and EventId members are ignored).
	 * @param OutActivityId				Populated with the ID of this activity in the database.
	 * @param OutTransactionEventId		Populated with the ID of the transaction event in the database (@see GetTransactionEvent).
	 *
	 * @return True if the transaction activity was added, false otherwise.
	 */
	UE_API bool AddTransactionActivity(const FConcertSyncTransactionActivity& InTransactionActivity, int64& OutActivityId, int64& OutTransactionEventId);

	/**
	 * Add a new package activity to this database, assigning it both an activity and package event ID.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the server to populate its version of the session database.
	 *
	 * @param InPackageActivity			The base part of the package activity to add (the ActivityId, EventTime, EventType, EventId, and PackageRevision members are ignored).
	 * @param InPackageInfo				Information about the package affected.
	 * @param InPackageDataStream		A stream to the package data if the activty is about adding or modifying a package. Can be empty when the activity is about deleting or discarding changes.
	 * @param OutActivityId				Populated with the ID of this activity in the database.
	 * @param OutPackageEventId			Populated with the ID of the package event in the database (@see GetPackageEvent).
	 *
	 * @return True if the package activity was added, false otherwise.
	 */
	UE_API bool AddPackageActivity(const FConcertSyncActivity& InPackageActivity, const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream, int64& OutActivityId, int64& OutPackageEventId);

	/**
	 * Add a new replication activity to this database, assigning it both an activity and lock event ID.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the server to populate its version of the session database.
	 *
	 * @param InReplicationActivity		The lock activity to add (the ActivityId, EventTime, EventType, and EventId members are ignored).
	 * @param OutActivityId				Populated with the ID of this activity in the database.
	 * @param OutReplicationEventId			Populated with the ID of the lock event in the database (@see GetLockEvent).
	 *
	 * @return True if the lock activity was added, false otherwise.
	 */
	UE_API bool AddReplicationActivity(const FConcertSyncReplicationActivity& InReplicationActivity, int64& OutActivityId, int64& OutReplicationEventId);
	
	/**
	 * Iterates the given range, calls UpdateCallback on each element, and commits the update.
	 */
	UE_API bool SetActivities(const TSet<int64>& ActivityIds, TFunctionRef<void(FConcertSyncActivity&)> UpdateCallback);
	
	/**
	 * Set a connection activity in this database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the client to populate its version of the session database from data synced from the server.
	 *
	 * @param InConnectionActivity		The connection activity to set.
	 *
	 * @return True if the connection activity was set, false otherwise.
	 */
	UE_API bool SetConnectionActivity(const FConcertSyncConnectionActivity& InConnectionActivity);

	/**
	 * Set a lock activity in this database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the client to populate its version of the session database from data synced from the server.
	 *
	 * @param InLockActivity			The lock activity to set.
	 *
	 * @return True if the lock activity was set, false otherwise.
	 */
	UE_API bool SetLockActivity(const FConcertSyncLockActivity& InLockActivity);

	/**
	 * Set a transaction activity in this database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the client to populate its version of the session database from data synced from the server.
	 *
	 * @param InTransactionActivity		The transaction activity to set.
	 * @param bMetaDataOnly				True to store the meta data only, omitting the transaction data required to replay the transaction.
	 *
	 * @return True if the transaction activity was set, false otherwise.
	 */
	UE_API bool SetTransactionActivity(const FConcertSyncTransactionActivity& InTransactionActivity, const bool bMetaDataOnly = false);

	/**
	 * Set a package activity in this database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the client to populate its version of the session database from data synced from the server.
	 *
	 * @param InPackageActivity			The base part of the package activity to set.
	 * @param InPackageActivityEvent	The event meta data (package info + revision) and possibly the corresponding package data.
	 * @param bMetaDataOnly				True to store the meta data only, omitting the package data itself.
	 *
	 * @return True if the package activity was set, false otherwise.
	 */
	UE_API bool SetPackageActivity(const FConcertSyncActivity& InPackageActivity, FConcertSyncPackageEventData& InPackageActivityEvent, const bool bMetaDataOnly = false);

	/**
	 * Set a replication activity in this database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 * @note This function is expected to be called on the client to populate its version of the session database from data synced from the server.
	 *
	 * @param InReplicationActivity		The replication activity to set.
	 *
	 * @return True if the transaction activity was set, false otherwise.
	 */
	UE_API bool SetReplicationActivity(const FConcertSyncReplicationActivity& InReplicationActivity);
	
	/**
	 * Get the generic part of an activity from this database.
	 *
	 * @param InActivityId				The ID of the activity to find.
	 * @param OutActivity				The generic part of an activity to populate with the result.
	 *
	 * @return True if the activity was found, false otherwise.
	 */
	UE_API bool GetActivity(const int64 InActivityId, FConcertSyncActivity& OutActivity) const;

	/**
	 * Get a connection activity from this database.
	 *
	 * @param InActivityId				The ID of the activity to find.
	 * @param OutConnectionActivity		The connection activity to populate with the result.
	 *
	 * @return True if the connection activity was found, false otherwise.
	 */
	UE_API bool GetConnectionActivity(const int64 InActivityId, FConcertSyncConnectionActivity& OutConnectionActivity) const;

	/**
	 * Get a lock activity from this database.
	 *
	 * @param InActivityId				The ID of the activity to find.
	 * @param OutLockActivity			The lock activity to populate with the result.
	 *
	 * @return True if the lock activity was found, false otherwise.
	 */
	UE_API bool GetLockActivity(const int64 InActivityId, FConcertSyncLockActivity& OutLockActivity) const;

	/**
	 * Get a transaction activity from this database.
	 *
	 * @param InActivityId				The ID of the activity to find.
	 * @param OutTransactionActivity	The transaction activity to populate with the result.
	 *
	 * @return True if the transaction activity was found, false otherwise.
	 */
	UE_API bool GetTransactionActivity(const int64 InActivityId, FConcertSyncTransactionActivity& OutTransactionActivity) const;

	/**
	 * Get a package activity from this database.
	 *
	 * @param InActivityId			The ID of the activity to find.
	 * @param PackageActivityFn		A callback invoked with the package activity components (the base activity part and the event part).
	 * @note The package data stream passed to the callback through the FConcertSycnPackageEventData is only valid during the callback.
	 *
	 * @return True if the package activity was found, false otherwise.
	 */
	UE_API bool GetPackageActivity(const int64 InActivityId, FConsumePackageActivityFunc PackageActivityFn) const;

	/**
	 * Get a replication activity from this database.
	 *
	 * @param InActivityId				The ID of the activity to find.
	 * @param OutReplicationActivity	The replication activity to populate with the result.
	 *
	 * @return True if the replication activity was found, false otherwise.
	 */
	UE_API bool GetReplicationActivity(const int64 InActivityId, FConcertSyncReplicationActivity& OutReplicationActivity) const;

	/**
	 * Get a replication activity from this database by its event ID
	 *
	 * @param InEventId					The ID of the event that is saved in a replication activity.
	 * @param OutReplicationActivity	The replication activity to populate with the result.
	 *
	 * @return True if the replication activity was found, false otherwise.
	 */
	UE_API bool GetReplicationActivityForEvent(const int64 InEventId, FConcertSyncReplicationActivity& OutReplicationActivity) const;
	
	/**
	 * Get the type of an activity in this database.
	 *
	 * @param InActivityId				The ID of the activity to query.
	 * @param OutEventType				The event type to populate with the result.
	 *
	 * @return True if the activity was found, false otherwise.
	 */
	UE_API bool GetActivityEventType(const int64 InActivityId, EConcertSyncActivityEventType& OutEventType) const;
	
	/**
	 * Get the generic part of an activity for an event in this database.
	 *
	 * @param InEventId					The ID of the event to find the activity for.
	 * @param InEventType				The type of the event to find the activity for.
	 * @param OutActivity				The generic part of an activity to populate with the result.
	 *
	 * @return True if the activity was found, false otherwise.
	 */
	UE_API bool GetActivityForEvent(const int64 InEventId, const EConcertSyncActivityEventType InEventType, FConcertSyncActivity& OutActivity) const;

	/**
	 * Get a connection activity for an event in this database.
	 *
	 * @param InConnectionEventId		The ID of the connection event to find the activity for.
	 * @param OutConnectionActivity		The connection activity to populate with the result.
	 *
	 * @return True if the connection activity was found, false otherwise.
	 */
	UE_API bool GetConnectionActivityForEvent(const int64 InConnectionEventId, FConcertSyncConnectionActivity& OutConnectionActivity) const;

	/**
	 * Get a lock activity for an event in this database.
	 *
	 * @param InLockEventId				The ID of the lock event to find the activity for.
	 * @param OutLockActivity			The lock activity to populate with the result.
	 *
	 * @return True if the lock activity was found, false otherwise.
	 */
	UE_API bool GetLockActivityForEvent(const int64 InLockEventId, FConcertSyncLockActivity& OutLockActivity) const;

	/**
	 * Get a transaction activity for an event in this database.
	 *
	 * @param InTransactionEventId		The ID of the transaction event to find the activity for.
	 * @param OutTransactionActivity	The transaction activity to populate with the result.
	 *
	 * @return True if the transaction activity was found, false otherwise.
	 */
	UE_API bool GetTransactionActivityForEvent(const int64 InTransactionEventId, FConcertSyncTransactionActivity& OutTransactionActivity) const;

	/**
	 * Get a package activity for an event in this database.
	 *
	 * @param InPackageEventId		The ID of the package event to find the activity for.
	 * @param PackageActivityFn		A callback invoked with the package activity components (the base activity part and the event part).
	 * @note The package data stream passed to the callback through the FConcertSycnPackageEventData is only valid during the callback.
	 *
	 * @return True if the package activity was found, false otherwise.
	 */
	UE_API bool GetPackageActivityForEvent(const int64 InPackageEventId, FIteratePackageActivityFunc PackageActivityFn) const;

	/**
	 * Enumerate the generic part of the activities in this database.
	 *
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateActivities(FIterateActivityFunc InCallback) const;

	/**
	 * Enumerate all the connection activities in this database.
	 *
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the connection activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateConnectionActivities(TFunctionRef<bool(FConcertSyncConnectionActivity&&)> InCallback) const;

	/**
	 * Enumerate all the lock activities in this database.
	 *
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the lock activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateLockActivities(TFunctionRef<bool(FConcertSyncLockActivity&&)> InCallback) const;

	/**
	 * Enumerate all the transaction activities in this database.
	 *
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the transaction activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateTransactionActivities(TFunctionRef<bool(FConcertSyncTransactionActivity&&)> InCallback) const;

	/**
	 * Enumerate all the package activities in this database.
	 *
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the package activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumeratePackageActivities(FIteratePackageActivityFunc InCallback) const;

	/**
	 * Enumerate all the replication activities in this database.
	 *
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the replication activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateReplicationActivities(TFunctionRef<bool(FConcertSyncReplicationActivity&&)> InCallback) const;

	/**
	 * Enumerate all the activities in this database of the given type.
	 *
	 * @param InEventType				The type of the event to enumerate the activities for.
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateActivitiesForEventType(const EConcertSyncActivityEventType InEventType, FIterateActivityFunc InCallback) const;

	/**
	 * Enumerate all the activities in this database in the given range.
	 *
	 * @param InFirstActivityId			The first activity ID to include in the results.
	 * @param InMaxNumActivities		The maximum number of activities to include in the results.
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateActivitiesInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, FIterateActivityFunc InCallback) const;

	/**
	 * Enumerate the IDs and event types of all the activities in this database.
	 *
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateActivityIdsAndEventTypes(TFunctionRef<bool(int64, EConcertSyncActivityEventType)> InCallback) const;

	/**
	 * Enumerate the IDs and event types of the activities in this database in the given range.
	 *
	 * @param InFirstActivityId			The first activity ID to include in the results.
	 * @param InMaxNumActivities		The maximum number of activities to include in the results.
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateActivityIdsAndEventTypesInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, TFunctionRef<bool(int64, EConcertSyncActivityEventType)> InCallback) const;

	/**
	 * Enumerate the IDs, event types and flags of the activities in this database in the given range.
	 *
	 * @param InFirstActivityId			The first activity ID to include in the results.
	 * @param InMaxNumActivities		The maximum number of activities to include in the results.
	 * @param InCallback				Callback invoked for each activity; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the activities were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateActivityIdsWithEventTypesAndFlagsInRange(const int64 InFirstActivityId, const int64 InMaxNumActivities, TFunctionRef<bool(int64, EConcertSyncActivityEventType, EConcertSyncActivityFlags)> InCallback) const;
	
	/**
	 * Get the maximum ID of the activities in this database.
	 *
	 * @param OutActivityId				The activity ID to populate with the result.
	 *
	 * @return True if the activity ID was resolved, false otherwise.
	 */
	UE_API bool GetActivityMaxId(int64& OutActivityId) const;

	/**
	 * Set an endpoint in this database, creating or replacing it.
	 *
	 * @param InEndpointId				The ID of the endpoint to set.
	 * @param InEndpointData			The endpoint data to set.
	 *
	 * @return True if the endpoint was set, false otherwise.
	 */
	UE_API bool SetEndpoint(const FGuid& InEndpointId, const FConcertSyncEndpointData& InEndpointData);

	/**
	 * Get an endpoint from this database.
	 *
	 * @param InEndpointId				The ID of the endpoint to find.
	 * @param OutEndpointData			The endpoint data to populate with the result.
	 *
	 * @return True if the endpoint was found, false otherwise.
	 */
	UE_API bool GetEndpoint(const FGuid& InEndpointId, FConcertSyncEndpointData& OutEndpointData) const;

	/**
	 * Enumerate all the endpoints in this database.
	 *
	 * @param InCallback				Callback invoked for each endpoint; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the endpoints were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateEndpoints(TFunctionRef<bool(FConcertSyncEndpointIdAndData&&)> InCallback) const;

	/**
	 * Enumerate all the endpoint IDs in this database.
	 *
	 * @param InCallback				Callback invoked for each endpoint ID; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the endpoint IDs were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateEndpointIds(TFunctionRef<bool(FGuid)> InCallback) const;

	/**
	 * Get a connection event from this database.
	 *
	 * @param InConnectionEventId		The ID of the connection event to find.
	 * @param OutConnectionEvent		The connection event to populate with the result.
	 *
	 * @return True if the connection event was found, false otherwise.
	 */
	UE_API bool GetConnectionEvent(const int64 InConnectionEventId, FConcertSyncConnectionEvent& OutConnectionEvent) const;

	/**
	 * Get a lock event from this database.
	 *
	 * @param InLockEventId				The ID of the lock event to find.
	 * @param OutLockEvent				The lock event to populate with the result.
	 *
	 * @return True if the lock event was found, false otherwise.
	 */
	UE_API bool GetLockEvent(const int64 InLockEventId, FConcertSyncLockEvent& OutLockEvent) const;

	/**
	 * Get a transaction event from this database.
	 *
	 * @param InTransactionEventId		The ID of the transaction event to find.
	 * @param OutTransactionEvent		The transaction event to populate with the result.
	 * @param InMetaDataOnly			True to only get the meta-data for the transaction, and skip the bulk of the data.
	 *
	 * @return True if the transaction event was found, false otherwise.
	 */
	UE_API bool GetTransactionEvent(const int64 InTransactionEventId, FConcertSyncTransactionEvent& OutTransactionEvent, const bool InMetaDataOnly = false) const;

	/**
	 * Get a replication event from this database.
	 *
	 * @param InReplicationEventId		The ID of the replication event to find.
	 * @param OutReplicationEvent		The replication event to populate with the result.
	 *
	 * @return True if the replication event was found, false otherwise.
	 */
	UE_API bool GetReplicationEvent(const int64 InReplicationEventId, FConcertSyncReplicationEvent& OutReplicationEvent) const;

	/**
	 * Get the maximum ID of the transaction events in this database.
	 *
	 * @param OutTransactionEventId		The transaction event ID to populate with the result.
	 *
	 * @return True if the transaction event ID was resolved, false otherwise.
	 */
	UE_API bool GetTransactionMaxEventId(int64& OutTransactionEventId) const;

	/**
	 * Check whether the given transaction event ID is currently for a live transaction event.
	 *
	 * @param InTransactionEventId		The ID of the transaction event to check.
	 * @param OutIsLive					Bool to populate with the result.
	 *
	 * @return True if the transaction event ID was queried successfully, false otherwise.
	 */
	UE_API bool IsLiveTransactionEvent(const int64 InTransactionEventId, bool& OutIsLive) const;

	/**
	 * Get the IDs of any live transaction events.
	 *
	 * @param OutTransactionEventIds	The array of transaction event IDs to populate with the result (will be sorted in ascending order).
	 *
	 * @return True if the live transaction event IDs were resolved, false otherwise.
	 */
	UE_API bool GetLiveTransactionEventIds(TArray<int64>& OutTransactionEventIds) const;

	/**
	 * Get the IDs of any live transaction events for the given package name.
	 *
	 * @param InPackageName				The name of the package to get the live transaction event IDs for.
	 * @param OutTransactionEventIds	The array of transaction event IDs to populate with the result (will be sorted in ascending order).
	 *
	 * @return True if the live transaction event IDs were resolved, false otherwise.
	 */
	UE_API bool GetLiveTransactionEventIdsForPackage(const FName InPackageName, TArray<int64>& OutTransactionEventIds) const;

	/**
	 * Get if a Package has any live transactions
	 *
	 * @param InPackageName				The name of the package to check if it has live transactions.
	 * @param OutHasLiveTransaction		Bool to populate with the result.
	 *
	 * @return True if query was resolved correctly, false otherwise.
	 */
	UE_API bool PackageHasLiveTransactions(const FName InPackageName, bool& OutHasLiveTransaction) const;

	/**
	 * Enumerate the IDs of any live transaction events for the given package name.
	 *
	 * @param InPackageName				The name of the package to get the live transaction event IDs for.
	 * @param InCallback				Callback invoked for each transaction event ID; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the live transaction event IDs were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateLiveTransactionEventIdsForPackage(const FName InPackageName, TFunctionRef<bool(int64)> InCallback) const;

	/**
	 * Enumerate the names of of any packages that have live transaction events.
	 *
	 * @param InCallback				Callback invoked for each package name; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the package names were enumerated without error, false otherwise.
	 */
	UE_API bool EnumeratePackageNamesWithLiveTransactions(TFunctionRef<bool(FName)> InCallback) const;

	/**
	 * Add a new dummy package event to this database, assigning it a package event ID.
	 * @note These are use to fence live transactions without generating a full activity entry (eg, when saving packages locally on the client during a persist operation).
	 *
	 * @param InPackageName				The name of the package to add the event for.
	 * @param OutPackageEventId			Populated with the ID of the package event in the database.
	 *
	 * @return True if the package event was added, false otherwise.
	 */
	UE_API bool AddDummyPackageEvent(const FName InPackageName, int64& OutPackageEventId);

	/**
	 * Get a package event meta data (omitting the package data itself) from this database.
	 *
	 * @param InPackageEventId		The ID of the package event to find.
	 * @param OuptPackageRevision	The package revision number.
	 * @param OutPackageInfo		The Package info.
	 *
	 * @return True if the package event was found, false otherwise.
	 */
	UE_API bool GetPackageEventMetaData(const int64 InPackageEventId, int64& OuptPackageRevision, FConcertPackageInfo& OutPackageInfo) const;

	/**
	 * Get Package Event Id from Package Name.  The event id will be from the head revision of the package.
	 *
	 * @param InPackageName
	 * @param OutPackageEventId
	 *
	 * @return True if the package event was found, false otherwise
	 */
	UE_API bool GetPackageHeadEventFromName(const FName InPackageName, int64& OutPackageEventId);

	/**
	 * Get a package event from this database.
	 *
	 * @param InPackageEventId			The ID of the package event to find.
	 * @param PackageEventFn			The callback invoked with the package event containing the event meta data and the package data if available.
	 *
	 * @return True if the package event was found, false otherwise.
	 */
	UE_API bool GetPackageEvent(const int64 InPackageEventId, const TFunctionRef<void(FConcertSyncPackageEventData&)>& PackageEventFn) const;

	/**
	 * Enumerate package names for packages with a head revision (at least one package event)
	 *
	 * @param InCallback				Callback invoked for each package name; return true to continue enumeration, or false to stop.
	 * @param IgnorePersisted			Will skip enumeration of packages which head revision have been persisted.
	 *
	 * @return True if the package data was enumerated without error, false otherwise.
	 */
	UE_API bool EnumeratePackageNamesWithHeadRevision(TFunctionRef<bool(FName)> InCallback, bool IgnorePersisted) const;

	/**
	 * Enumerate the head revision package data for all packages in this database.
	 *
	 * @param InCallback				Callback invoked for each package; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the package data was enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateHeadRevisionPackageData(TFunctionRef<bool(const FConcertPackageInfo&, FConcertPackageDataStream&)> InCallback) const;

	/**
	 * Get the data from this database for the given package name for the given revision.
	 *
	 * @param InPackageName				The name of the package to get the head revision for.
	 * @param OutPackageInfo			The package info to populate with the result.
	 * @param InPackageRevision			The revision of the package to get the data for, or null to get the head revision.
	 *
	 * @return True if package data could be found for the given revision, false otherwise.
	 */
	UE_API bool GetPackageInfoForRevision(const FName InPackageName, FConcertPackageInfo& OutPackageInfo, const int64* InPackageRevision = nullptr) const;
	
	/**
	 * Get the data from this database for the given package name for the given revision.
	 *
	 * @param InPackageName				The name of the package to get the head revision for.
	 * @param InCallback				A callback invoked with the package info and package data.
	 * @param InPackageRevision			The revision of the package to get the data for, or null to get the head revision.
	 *
	 * @return True if package data could be found for the given revision, false otherwise.
	 */
	UE_API bool GetPackageDataForRevision(const FName InPackageName, const TFunctionRef<void(const FConcertPackageInfo&, FConcertPackageDataStream&)>& InCallback, const int64* InPackageRevision = nullptr) const;

	/**
	 * Gets the package size in bytes of a package
	 *
	 * @param InPackageName				The name of the package to get the head revision for.
	 * @param InPackageRevision			The revision of the package to get the data for, or null to get the head revision.
	 *
	 * @return True if package data could be found for the given revision, false otherwise.
	 */
	UE_API TOptional<int64> GetPackageSizeForRevision(const FName InPackageName, const int64* InPackageRevision = nullptr) const;
	
	/**
	 * Get the head revision in this database for the given package name.
	 *
	 * @param InPackageName				The name of the package to get the head revision for.
	 * @param OutRevision				The package revision to populate with the result.
	 *
	 * @return True if the head revision was resolved, false otherwise.
	 */
	UE_API bool GetPackageHeadRevision(const FName InPackageName, int64& OutRevision) const;

	/**
	 * Check whether the given package event ID is currently for the head revision of the package in the event.
	 *
	 * @param InPackageEventId			The ID of the package event to check.
	 * @param OutIsHeadRevision			Bool to populate with the result.
	 *
	 * @return True if the package event ID was queried successfully, false otherwise.
	 */
	UE_API bool IsHeadRevisionPackageEvent(const int64 InPackageEventId, bool& OutIsHeadRevision) const;

	/**
	 * Get the maximum ID of the replication events in this database that was triggered by EndpointId and had ActivityType.
	 *
	 * @param EndpointId				The ID of the client that created the activity
	 * @param ActivityType				The activity type the replication activity should have 
	 * @param OutReplicationEventId		The replication event ID to populate with the result.
	 *
	 * @return True if the replication event ID was resolved, false otherwise.
	 */
	UE_API bool GetReplicationMaxEventIdByClientAndType(const FGuid& EndpointId, EConcertSyncReplicationActivityType ActivityType, int64& OutReplicationEventId) const;

	/**
	 * Add a package event ID for the head revision to the persist events in this database, if not already existing.
	 *
	 * @param PackageName				The package name to add an event for.
	 * @param OutPersistEventId			Populated with the ID of the persist event in the database.
	 *
	 * @return True if the persist event was added, false otherwise.
	 */
	UE_API bool AddPersistEventForHeadRevision(FName InPackageName, int64& OutPersistEventId);

	/**
	 * Update the specified transaction event.
	 * @note The function is meant to update a transaction event that was partially synced to store the corresponding transaction data.
	 * @param InTransactionEventId		The ID of the event to update.
	 * @param InTransactionEvent		The event to store.
	 * @return True if the transaction was updated.
	 */
	UE_API bool UpdateTransactionEvent(const int64 InTransactionEventId, const FConcertSyncTransactionEvent& InTransactionEvent);

	/**
	 * Update the specified package event.
	 * @note The function is meant to be update a partially synced package event to store the corresponding package data.
	 * @param InPackageEventId			The ID of the event to update.
	 * @param PackageEvent				The event to store.
	 * @return True if the package was updated.
	 */
	UE_API bool UpdatePackageEvent(const int64 InPackageEventId, FConcertSyncPackageEventData& PackageEvent);

	/**
	 * Check Asynchronous Tasks Status
	 */
	UE_API void UpdateAsynchronousTasks();

	/**
	 * Flush any ongoing asynchronous tasks.
	 */
	UE_API void FlushAsynchronousTasks();

	/**
	 * Has all asynchronous package tasks completed.
	 */
	UE_API bool HasWritePackageTasksCompleted() const;

	FOnActivityProduced& OnActivityProduced() { return ActivityProducedEvent; }
	
private:

	using FProcessPackageRequest = TFunctionRef<bool(const FConcertPackageInfo& PackageInfo, const FString& DataFilename)>;
	/** Helper function which obtains package information and passes it to HandleFunc. */
	UE_API bool HandleRequestPackageRequest(const FName InPackageName, const int64* InPackageRevision, FProcessPackageRequest HandleFunc) const;

	/** Helper functions that for getting a package revision for an optional package revision argument */
	UE_API TOptional<int64> GetSpecifiedOrHeadPackageRevision(FName InPackageName, const int64* InPackageRevision) const;

	/**
	 * Schedule an asynchronous write for the given Package Stream.  The stream must be in-memory. File sharing
	 * asynchronous write is not supported.
	 *
	 * @param InPackageName	            Name of the package in content area.
	 * @param InDstPackageBlobPathName  Full path of the destination package.
	 * @param InPackageDataStream       The package data stream.
	 **/
	UE_API void ScheduleAsyncWrite(const FName& InPackageName, const FString& InDstPackageBlobPathname, FConcertPackageDataStream& InPackageDataStream);

	/**
	 * Set the active ignored state for the given activity.
	 *
	 * @param InActivityId				The ID of the activity to update.
	 * @param InIsIgnored				True if this activity should be ignored, false otherwise.
	 *
	 * @return True if the ignored state was set, false otherwise.
	 */
	UE_API bool SetActivityIgnoredState(const int64 InActivityId, const bool InIsIgnored);

	/**
	 * Add a new connection event to this database, assigning it a connection event ID.
	 *
	 * @param InConnectionEvent			The connection event to add.
	 * @param OutConnectionEventId		Populated with the ID of the connection event in the database.
	 *
	 * @return True if the connection event was added, false otherwise.
	 */
	UE_API bool AddConnectionEvent(const FConcertSyncConnectionEvent& InConnectionEvent, int64& OutConnectionEventId);

	/**
	 * Set a connection event in this database, creating or replacing it.
	 *
	 * @param InConnectionEventId		The ID of the connection event to set.
	 * @param InConnectionEvent			The connection event to set.
	 *
	 * @return True if the connection event was set, false otherwise.
	 */
	UE_API bool SetConnectionEvent(const int64 InConnectionEventId, const FConcertSyncConnectionEvent& InConnectionEvent);

	/**
	 * Add a new lock event to this database, assigning it a lock event ID.
	 *
	 * @param InLockEvent				The lock event to add.
	 * @param OutLockEventId			Populated with the ID of the lock event in the database.
	 *
	 * @return True if the lock event was added, false otherwise.
	 */
	UE_API bool AddLockEvent(const FConcertSyncLockEvent& InLockEvent, int64& OutLockEventId);

	/**
	 * Set a lock event in this database, creating or replacing it.
	 *
	 * @param InLockEventId				The ID of the lock event to set.
	 * @param InLockEvent				The lock event to set.
	 *
	 * @return True if the lock event was set, false otherwise.
	 */
	UE_API bool SetLockEvent(const int64 InLockEventId, const FConcertSyncLockEvent& InLockEvent);

	/**
	 * Add a new transaction event to this database, assigning it a transaction event ID.
	 *
	 * @param InTransactionEvent		The transaction event to add.
	 * @param OutTransactionEventId		Populated with the ID of the transaction event in the database.
	 *
	 * @return True if the transaction event was added, false otherwise.
	 */
	UE_API bool AddTransactionEvent(const FConcertSyncTransactionEvent& InTransactionEvent, int64& OutTransactionEventId);

	/**
	 * Set a transaction event in this database, creating or replacing it.
	 *
	 * @param InTransactionEventId		The ID of the transaction event to set.
	 * @param InTransactionEvent		The transaction event to set.
	 * @param bMetaDataOnly				True to store the meta data only, omitting the transaction data.
	 *
	 * @return True if the transaction event was set, false otherwise.
	 */
	UE_API bool SetTransactionEvent(const int64 InTransactionEventId, const FConcertSyncTransactionEvent& InTransactionEvent, const bool bMetaDataOnly = false);

	/**
	 * Enumerate the IDs of any live transaction events for the given package name ID.
	 *
	 * @param InPackageNameId			The package name ID to get the live transaction event IDs for.
	 * @param InCallback				Callback invoked for each transaction event ID; return true to continue enumeration, or false to stop.
	 *
	 * @return True if the live transaction event IDs were enumerated without error, false otherwise.
	 */
	UE_API bool EnumerateLiveTransactionEventIdsForPackage(const int64 InPackageNameId, TFunctionRef<bool(int64)> InCallback) const;

	/**
	 * Add a new package event to this database, assigning it a package event ID.
	 *
	 * @param InPackageInfo				The package info to add for this event.
	 * @param InPackageDataStream		The package data to add for this event (may be empty if the event doesn't carry package data).
	 * @param OutPackageEventId			Populated with the ID of the package event in the database.
	 *
	 * @return True if the package event was added, false otherwise.
	 */
	UE_API bool AddPackageEvent(const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream, int64& OutPackageEventId);

	/**
	 * Set a package event in this database, creating or replacing it.
	 *
	 * @param InPackageEventId			The ID of the package event to set.
	 * @param PackageRevision			The package revision number.
	 * @param PackageInfo				The package info.
	 * @param InPackageDataStream		The package data stream or null to set only the package meta data (revision and package info).
	 *
	 * @return True if the package event was set, false otherwise.
	 */
	UE_API bool SetPackageEvent(const int64 InPackageEventId, int64 PackageRevision, const FConcertPackageInfo& PackageInfo, FConcertPackageDataStream* InPackageDataStream);

	/**
	 * Add a new replication event to this database, assigning it a replication event ID.
	 *
	 * @param InReplicationEvent		The transaction event to add.
	 * @param OutReplicationEventId		Populated with the ID of the transaction event in the database.
	 *
	 * @return True if the replication event was added, false otherwise.
	 */
	UE_API bool AddReplicationEvent(const FConcertSyncReplicationEvent& InReplicationEvent, int64& OutReplicationEventId);

	/**
	 * Set a replication event in this database, creating or replacing it.
	 *
	 * @param InReplicationEventId		The ID of the replication event to set.
	 * @param InReplicationEvent		The replication event to set.
	 *
	 * @return True if the replication event was set, false otherwise.
	 */
	UE_API bool SetReplicationEvent(const int64 InReplicationEventId, const FConcertSyncReplicationEvent& InReplicationEvent);
	
	/**
	 * Get the maximum ID of the package events in this database.
	 *
	 * @param OutPackageId				The package event ID to populate with the result.
	 *
	 * @return True if the package event ID was resolved, false otherwise.
	 */
	UE_API bool GetPackageMaxEventId(int64& OutPackageEventId) const;

	/**
	 * Get the object path name from this database that matches the given object name ID.
	 *
	 * @param InObjectNameId			The ID of the object path name to get the result for.
	 * @param OutObjectPathName			The object path name to populate with the result.
	 *
	 * @return True if the object path name was resolved, false otherwise.
	 */
	UE_API bool GetObjectPathName(const int64 InObjectNameId, FName& OutObjectPathName) const;

	/**
	 * Get the object name ID from this database that matches the given object path name.
	 *
	 * @param InObjectPathName			The object path name to get the result for.
	 * @param OutObjectNameId			The object name ID to populate with the result.
	 *
	 * @return True if the object name ID name was resolved, false otherwise.
	 */
	UE_API bool GetObjectNameId(const FName InObjectPathName, int64& OutObjectNameId) const;

	/**
	 * Ensure that an object name ID in this database matches the given object path name, creating it if required.
	 *
	 * @param InObjectPathName			The object path name to get the result for.
	 * @param OutObjectNameId			The object name ID to populate with the result.
	 *
	 * @return True if the object name ID name was resolved, false otherwise.
	 */
	UE_API bool EnsureObjectNameId(const FName InObjectPathName, int64& OutObjectNameId);

	/**
	 * Get the package name from this database that matches the given package name ID.
	 *
	 * @param InPackageNameId			The ID of the package name to get the result for.
	 * @param OutPackageName			The package name to populate with the result.
	 *
	 * @return True if the package name was resolved, false otherwise.
	 */
	UE_API bool GetPackageName(const int64 InPackageNameId, FName& OutPackageName) const;

	/**
	 * Get the package name ID from this database that matches the given package name.
	 *
	 * @param InPackageName				The package name to get the result for.
	 * @param OutPackageNameId			The package name ID to populate with the result.
	 *
	 * @return True if the package name ID name was resolved, false otherwise.
	 */
	UE_API bool GetPackageNameId(const FName InPackageName, int64& OutPackageNameId) const;

	/**
	 * Ensure that a package name ID in this database matches the given package name, creating it if required.
	 *
	 * @param InPackageName				The package name to get the result for.
	 * @param OutPackageNameId			The package name ID to populate with the result.
	 *
	 * @return True if the package name ID name was resolved, false otherwise.
	 */
	UE_API bool EnsurePackageNameId(const FName InPackageName, int64& OutPackageNameId);

	/**
	 * Map the resource names so that they have an association with the given lock event ID in this database.
	 *
	 * @param InLockEventId				The lock event ID to map resource references to.
	 * @param InResourceNames			The array of resource names to map.
	 *
	 * @return True if the resource name references were mapped, false otherwise.
	 */
	UE_API bool MapResourceNamesForLock(const int64 InLockEventId, const TArray<FName>& InResourceNames);

	/**
	 * Map the package names in the given transaction so that they have an association with the given transaction event ID in this database.
	 *
	 * @param InTransactionEventId		The transaction event ID to map package references to.
	 * @param InTransactionEvent		The transaction containing potential package references.
	 *
	 * @return True if the package name references were mapped, false otherwise.
	 */
	UE_API bool MapPackageNamesForTransaction(const int64 InTransactionEventId, const FConcertTransactionFinalizedEvent& InTransactionEvent);

	/**
	 * Map the object path names in the given transaction so that they have an association with the given transaction event ID in this database.
	 *
	 * @param InTransactionEventId		The transaction event ID to map object references to.
	 * @param InTransactionEvent		The transaction containing potential object references.
	 *
	 * @return True if the object path name references were mapped, false otherwise.
	 */
	UE_API bool MapObjectNamesForTransaction(const int64 InTransactionEventId, const FConcertTransactionFinalizedEvent& InTransactionEvent);

	/**
	 * Save and cache the given transaction data with the given filename.
	 *
	 * @param InTransactionFilename		The filename to save the transaction data as.
	 * @param InTransaction				The transaction data to save.
	 *
	 * @return True if the transaction data was saved, false otherwise.
	 */
	UE_API bool SaveTransaction(const FString& InTransactionFilename, const FStructOnScope& InTransaction) const;

	/**
	 * Load and cache the transaction data for the given filename.
	 *
	 * @param InTransactionFilename		The filename to load the transaction data from.
	 * @param OutTransaction			The transaction data to populate with the result.
	 *
	 * @return True if the transaction data was loaded, false otherwise.
	 */
	UE_API bool LoadTransaction(const FString& InTransactionFilename, FStructOnScope& OutTransaction) const;

	/**
	 * Save and cache the given package data with the given filename.
	 *
	 * @param InDstPackageBlobPathname	The pathname to save the package data as. The package data may or may not be compressed.
	 * @param InPackageInfo				The package info to save.
	 * @param InPackageDataStream		The package data to save.
	 *
	 * @return True if the package data was saved, false otherwise.
	 */
	UE_API bool SavePackage(const FString& InDstPackageBlobPathname, const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream);

	/**
	 * Load the package data for the given filename.
	 *
	 * @param InPackageBlobFilename			The blob filename containing the package data to extract.
	 * @param PackageDataStreamFn			Callback invoked to let the caller stream the package data.
	 * @note The archive passed as output is only valid during the callback, its position is set at the beginning of the data (not necessarily zero) and may contain data passed the supplied size.
	 * @return True if the package data was loaded, false otherwise.
	 */
	UE_API bool LoadPackage(const FString& InPackageBlobFilename, const TFunctionRef<void(FConcertPackageDataStream&)>& PackageDataStreamFn) const;
	
	/** Called when an activity is produced */
	FOnActivityProduced ActivityProducedEvent;
	
	/** Root path to store all session data under */
	FString SessionPath;

	/** In-memory cache of on-disk transaction files */
	TUniquePtr<FConcertFileCache> TransactionFileCache;

	/** In-memory cache of on-disk package files */
	TUniquePtr<FConcertFileCache> PackageFileCache;

	/** Prepared statements for the currently open database */
	TUniquePtr<FConcertSyncSessionDatabaseStatements> Statements;

	/** Internal SQLite database */
	TUniquePtr<FSQLiteDatabase> Database;

	TPimplPtr<struct FDeferredLargePackageIOImpl> DeferredLargePackageIOPtr;
};

namespace UE::ConcertSyncCore::SyncDatabase
{
	/** Get the global saved package delegate */
	CONCERTSYNCCORE_API FOnPackageSaved& GetOnPackageSavedDelegate();
}

namespace ConcertSyncSessionDatabaseFilterUtil
{

	/**
	 * Check to see whether the transaction event with the given ID passes the given filter.
	 *
	 * @param InTransactionEventId		The ID of the transaction event to query for.
	 * @param InSessionFilter			The session filter used to query the event against.
	 * @param InDatabase				The database to query data from.
	 *
	 * @return True if the transaction event passes the filter, false otherwise.
	 */
	CONCERTSYNCCORE_API bool TransactionEventPassesFilter(const int64 InTransactionEventId, const FConcertSessionFilter& InSessionFilter, const FConcertSyncSessionDatabase& InDatabase);

	/**
	 * Check to see whether the package event with the given ID passes the given filter.
	 *
	 * @param InPackageEventId			The ID of the package event to query for.
	 * @param InSessionFilter			The session filter used to query the event against.
	 * @param InDatabase				The database to query data from.
	 *
	 * @return True if the package event passes the filter, false otherwise.
	 */
	CONCERTSYNCCORE_API bool PackageEventPassesFilter(const int64 InPackageEventId, const FConcertSessionFilter& InSessionFilter, const FConcertSyncSessionDatabase& InDatabase);

} // namespace ConcertSyncSessionDatabaseFilterUtil

#undef UE_API
