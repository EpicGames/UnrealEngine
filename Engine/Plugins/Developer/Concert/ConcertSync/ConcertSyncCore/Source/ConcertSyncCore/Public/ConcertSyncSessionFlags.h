// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

enum class EConcertSyncSessionFlags : uint32
{
	None = 0,

	/**
	 * Flags controlling which features are enabled
	 */
	
	/** Enable live-syncing of data from the server to the client */
	EnableLiveSync = 1<<0,
	/** Enable historic connection activity tracking */
	EnableConnectionHistory = 1<<1,
	/** Enable live and historic lock activity tracking */
	EnableLocking = 1<<2,
	/** Enable live and historic transaction activity tracking */
	EnableTransactions = 1<<3,
	/** Enable live and historic package activity tracking */
	EnablePackages = 1<<4,
	/** Enable sequencer playback tracking */
	EnableSequencer = 1<<5,
	/** Enable client presence tracking */
	EnablePresence = 1<<6,
	/** Enable usage of file sharing to share large blobs (like package data) rather than embedding that data in message/event.*/
	EnableFileSharing = 1<<7,
	/** Enable high-performance property replication. Clients will send data to server and the server replicates it back to other clients interested in that data. */
	EnableReplication = 1<<8,

	/**
	 * Flags controlling behavior of enabled features
	 */
	
	/** Discard transactions when a package is unloaded, eg) from a content hot-reload or closing a world editor (requires EnableTransactions) */
	ShouldDiscardTransactionsOnPackageUnload = 1<<9,
	/** Send snapshots of multi-frame transactions (like drags) to other clients (requires EnableTransactions) */
	ShouldSendTransactionSnapshots = 1<<10,
	/** Send the pristine state of a package (as revision 0) before saving it for the first time (requires EnablePackages) */
	ShouldSendPackagePristineState = 1<<11,
	/** Send package auto-saves, in addition to user-saves (requires EnablePackages) */
	ShouldSendPackageAutoSaves = 1<<12,
	/** Use a sandbox to hold package writes during a session (requires EnablePackages) */
	ShouldUsePackageSandbox = 1<<13,
	
	/**
	 * For Concert Replication, clients are allowed to globally mute objects.
	 * More formally, allows the use of FConcertReplication_ChangeMuteState_Requests.
	 */
	ShouldAllowGlobalMuting = 1<<14,
	/** Whether EConcertSyncActivityEventType::Replication activities should be generated. */
	ShouldEnableReplicationActivities = 1<<15,
	/** Whether clients are allowed to edit each other's streams & authority (via FConcertReplication_PutState_Request). */
	ShouldEnableRemoteEditing = 1 << 16,

	/**
	 * Defaults for different scenarios
	 */
	
	/** Default settings for a multi-user session */
	Default_MultiUserSession = EnableLiveSync | EnableConnectionHistory | EnableLocking | EnableTransactions | EnablePackages | EnableFileSharing | EnableSequencer | EnablePresence | ShouldSendTransactionSnapshots | ShouldUsePackageSandbox | EnableReplication | ShouldAllowGlobalMuting | ShouldEnableReplicationActivities | ShouldEnableRemoteEditing,
	/** Default settings for a disaster recovery session */
	Default_DisasterRecoverySession = EnableTransactions | EnablePackages | EnableFileSharing | ShouldDiscardTransactionsOnPackageUnload | ShouldSendPackagePristineState | ShouldSendPackageAutoSaves,
};
ENUM_CLASS_FLAGS(EConcertSyncSessionFlags);
