// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncSessionTypes.h"
#include "ConcertLogGlobal.h"

#include "Misc/PackageName.h"
#include "Replication/Messages/ReplicationActivity.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertSyncSessionTypes)

#define LOCTEXT_NAMESPACE "ConcertSyncActivitySummary"

namespace ActivitySummaryUtil
{
const FText BoldTextFormat = INVTEXT("<ActivityText.Bold>{0}</>");

FText ToRichTextBold(const FText& InArgs, bool bToRichText)
{
	return bToRichText ? FText::Format(BoldTextFormat, InArgs) : InArgs;
}

FText ToRichTextBold(const FString& InArgs, bool bToRichText)
{
	return ToRichTextBold(FText::FromString(InArgs), bToRichText);
}

FText ToRichTextBold(const FName& InArgs, bool bToRichText)
{
	return ToRichTextBold(FText::FromName(InArgs), bToRichText);
}

void DebugPrintExportedObject(const FConcertExportedObject& Object)
{
	UE_LOGF(LogConcert, Display, "==============================OBJECT===============================");
	UE_LOGF(LogConcert, Display, "Object ClassPath: %ls", *Object.ObjectId.ObjectClassPathName.ToString());
	UE_LOGF(LogConcert, Display, "Object Name: %ls", *Object.ObjectId.ObjectName.ToString());
	UE_LOGF(LogConcert, Display, "Object OuterPathName: %ls", *Object.ObjectId.ObjectOuterPathName.ToString());
	UE_LOGF(LogConcert, Display, "Object OwnerName: %ls", *FPackageName::ObjectPathToObjectName(Object.ObjectId.ObjectOuterPathName.ToString()));
	UE_LOGF(LogConcert, Display, "Package: %ls", *FPackageName::ObjectPathToPackageName(Object.ObjectId.ObjectOuterPathName.ToString()));

	if (Object.ObjectData.bAllowCreate)
	{
		UE_LOGF(LogConcert, Display, "AllowCreate: Yes");
	}
	if (Object.ObjectData.bIsPendingKill)
	{
		UE_LOGF(LogConcert, Display, "PendingKill: Yes");
	}
	if (!Object.ObjectData.NewPackageName.IsNone())
	{
		UE_LOGF(LogConcert, Display, "NewName: %ls", *Object.ObjectData.NewPackageName.ToString());
	}
	if (!Object.ObjectData.NewName.IsNone())
	{
		UE_LOGF(LogConcert, Display, "NewName: %ls", *Object.ObjectData.NewName.ToString());
	}
	if (!Object.ObjectData.NewOuterPathName.IsNone())
	{
		UE_LOGF(LogConcert, Display, "NewOuterPathName: %ls", *Object.ObjectData.NewOuterPathName.ToString());
	}
	if (!Object.ObjectData.NewExternalPackageName.IsNone())
	{
		UE_LOGF(LogConcert, Display, "NewPackageName: %ls", *Object.ObjectData.NewExternalPackageName.ToString());
	}
	if (Object.SerializedAnnotationData.Num())
	{
		UE_LOGF(LogConcert, Display, "Has Annotation");
	}
	for (const FConcertSerializedPropertyData& Prop : Object.PropertyDatas)
	{
		UE_LOGF(LogConcert, Display, "Property: %ls", *Prop.PropertyName.ToString());
	}
}

void DebugPrintExportedObject(const FString& Pathname, const FConcertExportedObject* Objects)
{
	if (Objects->ObjectData.bAllowCreate)
	{
		UE_LOGF(LogConcert, Display, "ObjectPathname: %ls, AllowCreate: Yes", *Pathname);
	}
	else if (Objects->ObjectData.bIsPendingKill)
	{
		UE_LOGF(LogConcert, Display, "ObjectPathname: %ls, PendingKill: Yes", *Pathname);
	}
	else
	{
		UE_LOGF(LogConcert, Display, "ObjectPathname: %ls", *Pathname);
	}
}

void DebugPrintExportedObjects(const FString::ElementType* Title, const TArray<TPair<const FString*, const FConcertExportedObject*>>& Objects)
{
	if (Objects.Num())
	{
		UE_LOGF(LogConcert, Display, "========================== %ls ==========================", Title);
		for (const TPair<const FString*, const FConcertExportedObject*>& Pair : Objects)
		{
			DebugPrintExportedObject(*Pair.Key, Pair.Value);
		}
	}
}

void DebugPrintExportedObjects(const FString::ElementType* Title, const TArray<TPair<FString, const FConcertExportedObject*>>& Objects)
{
	if (Objects.Num())
	{
		UE_LOGF(LogConcert, Display, "========================== %ls ==========================", Title);
		for (const TPair<FString, const FConcertExportedObject*>& Pair : Objects)
		{
			DebugPrintExportedObject(Pair.Key, Pair.Value);
		}
	}
}

FName GetObjectDisplayName(const FString& OuterPathName, const FName& ObjectName)
{
	FName ObjectDisplayName;

	// If this top object is a component of an actor or a sequence (Ex its outer path is an actor like /Game/FooMap.FooMap:PersistentLevel.Cube_1 -> OwnerName == "Cube_1")
	FString ObjectOwnerName = FPackageName::ObjectPathToObjectName(OuterPathName);
	if (ObjectOwnerName.Len() && ObjectOwnerName != TEXT("PersistentLevel"))
	{
		// Prefix the object name with its owner name (actor/sequence). (When adding an audio component to a cube for example, return "Cube_2.Audio rather than "Audio".
		ObjectDisplayName = *(ObjectOwnerName + TEXT(".") + ObjectName.ToString());
	}
	else
	{
		// The object is an actor/sequence (not a component of an actor or a sequence)
		ObjectDisplayName = ObjectName;
	}

	return ObjectDisplayName;
}

FName GetObjectDisplayName(const FString& ObjectPathName)
{
	const FString ObjectName = FPackageName::ObjectPathToObjectName(ObjectPathName);
	if (!ObjectName.Len() || ObjectName == ObjectPathName)
	{
		// This is just a package name
		return *ObjectPathName;
	}

	const FString OuterPathName = ObjectPathName.LeftChop(ObjectName.Len() + 1); // +1 for the delimiter between the outer and the object
	return GetObjectDisplayName(OuterPathName, *ObjectName);
}

} // namespace ActivitySummaryUtil


FText FConcertSyncActivitySummary::ToDisplayText(const FText InUserDisplayName, const bool InUseRichText) const
{
	return InUserDisplayName.IsEmpty()
		? CreateDisplayText(InUseRichText)
		: CreateDisplayTextForUser(InUserDisplayName, InUseRichText);
}

FText FConcertSyncActivitySummary::CreateDisplayText(const bool InUseRichText) const
{
	return LOCTEXT("CreateDisplayText_NotImplemented", "CreateDisplayText not implemented!");
}

FText FConcertSyncActivitySummary::CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const
{
	return CreateDisplayText(InUseRichText);
}


FConcertSyncConnectionActivitySummary FConcertSyncConnectionActivitySummary::CreateSummaryForEvent(const FConcertSyncConnectionEvent& InEvent)
{
	FConcertSyncConnectionActivitySummary ActivitySummary;
	ActivitySummary.ConnectionEventType = InEvent.ConnectionEventType;
	return ActivitySummary;
}

FText FConcertSyncConnectionActivitySummary::CreateDisplayText(const bool InUseRichText) const
{
	FText FormatPattern;
	switch (ConnectionEventType)
	{
	case EConcertSyncConnectionEventType::Connected:
		FormatPattern = LOCTEXT("CreateDisplayText_Connection_Connected", "Joined the session.");
		break;
	case EConcertSyncConnectionEventType::Disconnected:
		FormatPattern = LOCTEXT("CreateDisplayText_Connection_Disconnected", "Left the session.");
		break;
	default:
		checkf(false, TEXT("Unhandled EConcertSyncConnectionEventType in FConcertSyncConnectionActivitySummary!"));
		break;
	}
	return FormatPattern;
}

FText FConcertSyncConnectionActivitySummary::CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const
{
	FText FormatPattern;
	switch (ConnectionEventType)
	{
	case EConcertSyncConnectionEventType::Connected:
		FormatPattern = LOCTEXT("CreateDisplayTextForUser_Connection_Connected", "{UserName} joined the session.");
		break;
	case EConcertSyncConnectionEventType::Disconnected:
		FormatPattern = LOCTEXT("CreateDisplayTextForUser_Connection_Disconnected", "{UserName} left the session.");
		break;
	default:
		checkf(false, TEXT("Unhandled EConcertSyncConnectionEventType in FConcertSyncConnectionActivitySummary!"));
		break;
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("UserName"), ActivitySummaryUtil::ToRichTextBold(InUserDisplayName, InUseRichText));
	return FText::Format(FormatPattern, Arguments);
}

FConcertSyncLockActivitySummary FConcertSyncLockActivitySummary::CreateSummaryForEvent(const FConcertSyncLockEvent& InEvent)
{
	const FString ResourceNameStr = InEvent.ResourceNames.Num() > 0 ? InEvent.ResourceNames[0].ToString() : FString();

	FConcertSyncLockActivitySummary ActivitySummary;
	ActivitySummary.LockEventType = InEvent.LockEventType;
	ActivitySummary.PrimaryResourceName = ActivitySummaryUtil::GetObjectDisplayName(ResourceNameStr);
	ActivitySummary.PrimaryPackageName = *FPackageName::ObjectPathToPackageName(ResourceNameStr);
	ActivitySummary.NumResources = InEvent.ResourceNames.Num();
	return ActivitySummary;
}

FText FConcertSyncLockActivitySummary::CreateDisplayText(const bool InUseRichText) const
{
	const bool bHasPackage = !PrimaryPackageName.IsNone() && PrimaryResourceName != PrimaryPackageName;

	FText FormatPattern;
	if (NumResources == 1)
	{
		switch (LockEventType)
		{
		case EConcertSyncLockEventType::Locked:
			FormatPattern = bHasPackage 
				? LOCTEXT("CreateDisplayText_Lock_LockedOneWithPackage", "Locked {PrimaryResourceName} in {PrimaryPackageName}.") 
				: LOCTEXT("CreateDisplayText_Lock_LockedOne", "Locked {PrimaryResourceName}.");
			break;
		case EConcertSyncLockEventType::Unlocked:
			FormatPattern = bHasPackage 
				? LOCTEXT("CreateDisplayText_Lock_UnlockedOneWithPackage", "Unlocked {PrimaryResourceName} in {PrimaryPackageName}.") 
				: LOCTEXT("CreateDisplayText_Lock_UnlockedOne", "Unlocked {PrimaryResourceName}.");
			break;
		default:
			checkf(false, TEXT("Unhandled EConcertSyncLockEventType in FConcertSyncLockActivitySummary!"));
			break;
		}
	}
	else
	{
		switch (LockEventType)
		{
		case EConcertSyncLockEventType::Locked:
			FormatPattern = bHasPackage 
				? LOCTEXT("CreateDisplayText_Lock_LockedManyWithPackage", "Locked {PrimaryResourceName} in {PrimaryPackageName}, plus {NumOtherResources} other {NumOtherResources}|plural(one=resource,other=resources).") 
				: LOCTEXT("CreateDisplayText_Lock_LockedMany", "Locked {PrimaryResourceName}, plus {NumOtherResources} other {NumOtherResources}|plural(one=resource,other=resources).");
			break;
		case EConcertSyncLockEventType::Unlocked:
			FormatPattern = bHasPackage 
				? LOCTEXT("CreateDisplayText_Lock_UnlockedManyWithPackage", "Unlocked {PrimaryResourceName} in {PrimaryPackageName}, plus {NumOtherResources} other {NumOtherResources}|plural(one=resource,other=resources).") 
				: LOCTEXT("CreateDisplayText_Lock_UnlockedMany", "Unlocked {PrimaryResourceName}, plus {NumOtherResources} other {NumOtherResources}|plural(one=resource,other=resources).");
			break;
		default:
			checkf(false, TEXT("Unhandled EConcertSyncLockEventType in FConcertSyncLockActivitySummary!"));
			break;
		}
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("PrimaryResourceName"), ActivitySummaryUtil::ToRichTextBold(PrimaryResourceName, InUseRichText));
	Arguments.Add(TEXT("PrimaryPackageName"), ActivitySummaryUtil::ToRichTextBold(PrimaryPackageName, InUseRichText));
	Arguments.Add(TEXT("NumOtherResources"), NumResources - 1);
	return FText::Format(FormatPattern, Arguments);
}

FText FConcertSyncLockActivitySummary::CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const
{
	const bool bHasPackage = !PrimaryPackageName.IsNone() && PrimaryResourceName != PrimaryPackageName;

	FText FormatPattern;
	if (NumResources == 1)
	{
		switch (LockEventType)
		{
		case EConcertSyncLockEventType::Locked:
			FormatPattern = bHasPackage 
				? LOCTEXT("CreateDisplayTextForUser_Lock_LockedOneWithPackage", "{UserName} locked {PrimaryResourceName} in {PrimaryPackageName}.") 
				: LOCTEXT("CreateDisplayTextForUser_Lock_LockedOne", "{UserName} locked {PrimaryResourceName}.");
			break;
		case EConcertSyncLockEventType::Unlocked:
			FormatPattern = bHasPackage 
				? LOCTEXT("CreateDisplayTextForUser_Lock_UnlockedOneWithPackage", "{UserName} unlocked {PrimaryResourceName} in {PrimaryPackageName}.") 
				: LOCTEXT("CreateDisplayTextForUser_Lock_UnlockedOne", "{UserName} unlocked {PrimaryResourceName}.");
			break;
		default:
			checkf(false, TEXT("Unhandled EConcertSyncLockEventType in FConcertSyncLockActivitySummary!"));
			break;
		}
	}
	else
	{
		switch (LockEventType)
		{
		case EConcertSyncLockEventType::Locked:
			FormatPattern = bHasPackage 
				? LOCTEXT("CreateDisplayTextForUser_Lock_LockedManyWithPackage", "{UserName} locked {PrimaryResourceName} in {PrimaryPackageName}, plus {NumOtherResources} other {NumOtherResources}|plural(one=resource,other=resources).") 
				: LOCTEXT("CreateDisplayTextForUser_Lock_LockedMany", "{UserName} locked {PrimaryResourceName}, plus {NumOtherResources} other {NumOtherResources}|plural(one=resource,other=resources).");
			break;
		case EConcertSyncLockEventType::Unlocked:
			FormatPattern = bHasPackage 
				? LOCTEXT("CreateDisplayTextForUser_Lock_UnlockedManyWithPackage", "{UserName} unlocked {PrimaryResourceName} in {PrimaryPackageName}, plus {NumOtherResources} other {NumOtherResources}|plural(one=resource,other=resources).") 
				: LOCTEXT("CreateDisplayTextForUser_Lock_UnlockedMany", "{UserName} unlocked {PrimaryResourceName}, plus {NumOtherResources} other {NumOtherResources}|plural(one=resource,other=resources).");
			break;
		default:
			checkf(false, TEXT("Unhandled EConcertSyncLockEventType in FConcertSyncLockActivitySummary!"));
			break;
		}
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("UserName"), ActivitySummaryUtil::ToRichTextBold(InUserDisplayName, InUseRichText));
	Arguments.Add(TEXT("PrimaryResourceName"), ActivitySummaryUtil::ToRichTextBold(PrimaryResourceName, InUseRichText));
	Arguments.Add(TEXT("PrimaryPackageName"), ActivitySummaryUtil::ToRichTextBold(PrimaryPackageName, InUseRichText));
	Arguments.Add(TEXT("NumOtherResources"), NumResources - 1);
	return FText::Format(FormatPattern, Arguments);
}


FConcertSyncTransactionActivitySummary FConcertSyncTransactionActivitySummary::CreateSummaryForEvent(const FConcertSyncTransactionEvent& InEvent)
{
	auto FillActivitySummary = [&InEvent](const EConcertSyncTransactionActivitySummaryType InTransactionSummaryType, const FName InPrimaryObjectName, const FName InPrimaryPackageName, const FName InNewObjectName, const int32 InNumActions, FConcertSyncTransactionActivitySummary& OutActivitySummary)
	{
		OutActivitySummary.TransactionSummaryType = InTransactionSummaryType;
		OutActivitySummary.TransactionTitle = InEvent.Transaction.Title;
		OutActivitySummary.PrimaryObjectName = InPrimaryObjectName;
		OutActivitySummary.PrimaryPackageName = InPrimaryPackageName;
		OutActivitySummary.NewObjectName = InNewObjectName;
		OutActivitySummary.NumActions = InNumActions;
	};

	auto CalculateActivitySummary = [&InEvent, &FillActivitySummary](FConcertSyncTransactionActivitySummary& OutActivitySummary)
	{
		// The algorithm extracts the main stories from a transaction. Usually, the interesting parts of a transaction is when something are created, renamed,
		// deleted or modified. Multiple objects can be changed in the same transaction. While the algorithm below is good enough, it may not be perfect. Transactions
		// recorded for different type of objects have different exported object patterns and the algorithm relies on pattern to extract the most accurate stories.

		// This string is reused many times in the algorithm below.
		static const FString PersistentLevel(TEXT("PersistentLevel"));
		static const FName PersistentLevelFName(*PersistentLevel);

		// DEBUG: Uncomment to delimit the logs for this execution of the function.
		//UE_LOGF(LogConcert, Display, "==============================BEGIN TRANSACTION===============================");

		// Keeps objects of interest.
		TArray<TPair<FString, const FConcertExportedObject*>> PotentialObjects;

		// Find objects that can be used to generate the story, eliminating those knowns to be irrelevant.
		for (const FConcertExportedObject& Object : InEvent.Transaction.ExportedObjects)
		{
			// DEBUG: Uncomment to see exported objects to debug when a transaction doesn't produce the expected story.
			//ActivitySummaryUtil::DebugPrintExportedObject(Object);

			if (Object.ObjectId.ObjectName == PersistentLevelFName)
			{
				continue; // Ignore changes on the 'PersistentLevel' itself because we want to tell how it changed instead.
			}
			else if (Object.ObjectId.ObjectClassPathName == TEXT("/Script/Engine.Model"))
			{
				continue; // Ignore changes to "Engine.Model" class of objects. From observation, it seems they never carry an interesting story by themselves. (Could be false in the future)
			}
			else if (!Object.ObjectData.NewOuterPathName.IsNone()) // Owner name changed?
			{
				// Happens when an object is renamed and a sub-component outer pathname is updated to reflect its owner new name.
				// This is a 'by product' effect of renaming an object. Top story will be the top object being renamed.
				continue;
			}
			else if (!Object.ObjectData.NewName.IsNone() && !Object.ObjectData.bAllowCreate) // Something was renamed, but not as part of the creation. (Some objects (ex: 'Cube') get renamed on creation)
			{
				// This transaction is a rename of an object.
				FString ObjectOuterPathname = Object.ObjectId.ObjectOuterPathName.ToString();
				FName PackageName(*FPackageName::ObjectPathToPackageName(ObjectOuterPathname));
				FName OldObjectDisplayName = ActivitySummaryUtil::GetObjectDisplayName(ObjectOuterPathname, Object.ObjectId.ObjectName);
				FName NewObjectDisplayName = ActivitySummaryUtil::GetObjectDisplayName(ObjectOuterPathname, Object.ObjectData.NewName);
				FillActivitySummary(EConcertSyncTransactionActivitySummaryType::Renamed, OldObjectDisplayName, PackageName, NewObjectDisplayName, 1, OutActivitySummary);

				// Assuming it is not possible to get a 'rename' with other kind of transactions (delete, create, transform, etc), everything else in the transaction is not interesting or
				// linked to the object rename. Since we captured the story of the transaction and we don't expect to find any other, return.
				return;
			}

			// Some objects, for example the basic Cube, gets renamed on creation from a generic name like 'StaticMeshActor_1" to a more specific name like 'Cube_1".
			FName ObjectName = Object.ObjectData.NewName.IsNone() ? Object.ObjectId.ObjectName : Object.ObjectData.NewName;

			// Keep the selected object with 'pathname-like' name, analog to a file pathname, so that objects with the same outer get different 'pathname'. (Ex. /Game/Map.Map:PersistentLevel.Cube1, /Game/Map.Map:PersistentLevel.Sphere1)
			PotentialObjects.Emplace(Object.ObjectId.ObjectOuterPathName.ToString() + TEXT(".") + ObjectName.ToString(), &Object);
		}

		// Build object hierarchies, putting sub-object after their top level object. (ex: /Game/Map.Map:PersistentLevel.Cube1, /Game/Map.Map:PersistentLevel.Cube1.Audio, /Game/Map.Map:PersistentLevel.Cube1.StaticMesh, /Game/Map.Map:PersistentLevel.Sphere1, ...)
		PotentialObjects.Sort([](const TPair<FString, const FConcertExportedObject*>& Lhs, const TPair<FString, const FConcertExportedObject*>& Rhs) { return Lhs.Key < Rhs.Key; });

		// DEBUG: Print the potential objects once sorted.
		//ActivitySummaryUtil::DebugPrintExportedObjects(TEXT("Potential"), PotentialObjects);

		// Contains the object selected to generate a story. (Retaining sort order of PotentialObjects)
		TArray<TPair<const FString*, const FConcertExportedObject*>> DeletedObjects;
		TArray<TPair<const FString*, const FConcertExportedObject*>> CreatedObjects;
		TArray<TPair<const FString*, const FConcertExportedObject*>> ModifiedObjects;

		// Classify the objects by type of transaction.
		for (const TPair<FString, const FConcertExportedObject*>& PotentialPair : PotentialObjects)
		{
			if (PotentialPair.Value->ObjectData.bIsPendingKill) // Something was deleted?
			{
				DeletedObjects.Emplace(&PotentialPair.Key, PotentialPair.Value);
			}
			else if (PotentialPair.Value->ObjectData.bAllowCreate) // Something was created?
			{
				CreatedObjects.Emplace(&PotentialPair.Key, PotentialPair.Value);
			}
			else // Something was modified.
			{
				static const FString EngineTransient(TEXT("/Engine/Transient"));
				// If the modified object is the 'Level' itself, ignore it, we are more interested to say how it was modified.
				FString OuterPathname(PotentialPair.Value->ObjectId.ObjectOuterPathName.ToString());
				if (OuterPathname != EngineTransient && OuterPathname == FPackageName::ObjectPathToPackageName(OuterPathname))
				{
					continue; // Ex. /Game/Map == /Game/Map -> Skip
				}

				ModifiedObjects.Emplace(&PotentialPair.Key, PotentialPair.Value);
			}
		}

		// DEBUG: Print the classified objects.
		//ActivitySummaryUtil::DebugPrintExportedObjects(TEXT("Deleted"), DeletedObjects);
		//ActivitySummaryUtil::DebugPrintExportedObjects(TEXT("Created"), CreatedObjects);
		//ActivitySummaryUtil::DebugPrintExportedObjects(TEXT("Modified"), ModifiedObjects);

		// Contains selected objects to generate stories.
		TArray<TPair<const FString*, const FConcertExportedObject*>> StoryObjects;

		// This flag is set to true when a 'create' or 'delete' story is going to be published. In this case, all modified objects are assumed to be the sub-product of the 'create' or delete' story.
		bool bIgnoreModifiedObjects = false;

		// This is a special case -> See doc where used.
		auto ShouldIgnorePolyCreation = [](const TArray<TPair<const FString*, const FConcertExportedObject*>>& Objects) -> bool
		{
			for (const TPair<const FString*, const FConcertExportedObject*>& ObjectPair : Objects)
			{
				// If a 'geometry' like 'Linear Stair', 'Spiral Stair' is modified or deleted, one of the deleted/modified object will be a BrushComponent.
				if (ObjectPair.Value->ObjectId.ObjectClassPathName.ToString() == TEXT("/Script/Engine.BrushComponent"))
				{
					return true;
				}
			}
			return false;
		};

		if (DeletedObjects.Num()) // Object(s) were deleted.
		{
			const FString* TopLevelObjectPathname = nullptr;
			for (const TPair<const FString*, const FConcertExportedObject*>& PathnameObjectPair : DeletedObjects)
			{
				// Just keep top-level objects. Ignore sub-components of the top-level object.
				if (TopLevelObjectPathname == nullptr || !PathnameObjectPair.Key->StartsWith(*TopLevelObjectPathname)) // A sub-object pathname contains the parent pathname as prefix. (Like a file system)
				{
					TopLevelObjectPathname = PathnameObjectPair.Key;
					StoryObjects.Emplace(PathnameObjectPair);
					bIgnoreModifiedObjects = true; // All modified objects are assumed to be the sub-product of the 'delete'.
				}
			}
		}

		if (CreatedObjects.Num()) // Object(s) were created. (Note that converting an actor from one type to another use a delete/create sequence in the same transaction)
		{
			const FString* TopLevelObjectPathname = nullptr;
			for (const TPair<const FString*, const FConcertExportedObject*>& CreatedPair : CreatedObjects)
			{
				// Special case: Modifying or deleting a geometries like 'Linear Stair', 'Spiral Stair', 'Cone', 'Sphere', etc. creates a 'Polys' in the persistent level
				// as a side effect. Detect this case to ignore its creation and avoid creating an activity for it as the main story is the modification of the geometry.
				if (CreatedPair.Value->ObjectId.ObjectClassPathName.ToString() == TEXT("/Script/Engine.Polys"))
				{
					if (ShouldIgnorePolyCreation(ModifiedObjects) || ShouldIgnorePolyCreation(DeletedObjects))
					{
						continue;
					}
				}

				// Just keep top-level objects. Ignore sub-components of the top-level object.
				if (TopLevelObjectPathname == nullptr || !CreatedPair.Key->StartsWith(*TopLevelObjectPathname)) // A sub-object pathname contains the parent pathname as prefix. (Like a file system)
				{
					TopLevelObjectPathname = CreatedPair.Key;
					StoryObjects.Emplace(CreatedPair);
					bIgnoreModifiedObjects = true; // All modified objects are assumed to be the sub-product of the 'create'.
				}
			}
		}

		// If nothing was deleted or created, then something was modified.
		if (!bIgnoreModifiedObjects && ModifiedObjects.Num() > 0)
		{
			TArray<TPair<const FString*, const FConcertExportedObject*>> StandaloneObjects;

			// This function processes a set of object representing a hierarchy and determines which object should be used for a story and which for which the decision has to be postponed later.
			auto ProcessHierarchyGroup = [&StandaloneObjects, &StoryObjects](const TArray<TPair<const FString*, const FConcertExportedObject*>>& ObjectGroup)
			{
				if (ObjectGroup.Num() == 1) // The group only contains a potential top level. (No children)
				{
					StandaloneObjects.Emplace(ObjectGroup[0]); // Not clear yet the role of this object, need further analysis to decide. Keep it for later.
				}
				else if (ObjectGroup.Num() == 2) // The group contains a top level object + 1 sub-object
				{
					StoryObjects.Emplace(ObjectGroup[1]); // Keep the sub-object to display 'object.subObject was modified'.
				}
				else // The group contains a top level object + few sub-objects (more than 1)
				{
					StoryObjects.Emplace(ObjectGroup[0]); // Keep the top level object to display 'object was modified'.
				}
			};

			// Find object hierarchies. ex: /Game/Map.Map:PersistentLevel.Cube1, /Game/Map.Map:PersistentLevel.Cube1.Audio, /Game/Map.Map:PersistentLevel.Cube1.StaticMesh
			TArray<TPair<const FString*, const FConcertExportedObject*>> ObjectHierarchyGroup;
			for (const TPair<const FString*, const FConcertExportedObject*>& ModifiedPair : ModifiedObjects) // ModifiedObjects retained the sort order of PotentialObjects
			{
				if (ObjectHierarchyGroup.Num() == 0 || ModifiedPair.Key->StartsWith(*ObjectHierarchyGroup[0].Key)) // A top level object or a sub-object.
				{
					ObjectHierarchyGroup.Emplace(ModifiedPair);
				}
				else // End of the group, a new top level object was discovered.
				{
					ProcessHierarchyGroup(ObjectHierarchyGroup);

					// Start a new group with the new top level object.
					ObjectHierarchyGroup.Reset();
					ObjectHierarchyGroup.Add(ModifiedPair);
				}
			}

			// Ensure to deal with any left-over objects.
			if (ObjectHierarchyGroup.Num())
			{
				ProcessHierarchyGroup(ObjectHierarchyGroup);
			}

			// This function tries to select which sub-object name should be displayed in the story amongst a set of objects that are considered 'siblings' or at least
			// doesn't have a clear 'parent-children' relationship between themselves.
			auto SelectModifiedComponent = [](const TArray<TPair<const FString*, const FConcertExportedObject*>>& RelatedObjects)
			{
				// When multiple objects are related (part of the same object), try selecting the best sub-object for the story.
				for (const TPair<const FString*, const FConcertExportedObject*>& OwnerPair : RelatedObjects)
				{
					if (OwnerPair.Value->PropertyDatas.Num()) // Has property modified?
					{
						return OwnerPair; // This is a heuristic. Usually, if we have a modified properties, this is the interesting component.
					}
				}
				return RelatedObjects[0]; // Default: return the first.
			};

			// DEBUG: Print the remaining objects to process.
			//ActivitySummaryUtil::DebugPrintExportedObjects(TEXT("Standalone"), StandaloneObjects);

			// Walk the remaining objects, trying to find if they are related such as siblings.
			// Ex: /Game/Map.Map:PersistentLevel.LinearStairBrush_0.BrushComponent0, /Game/Map.Map:PersistentLevel.LinearStairBrush_0.LinearStairBuilder_0, /Game/New.New:PersistentLevel.LinearStairBrush_0.Model_0.Polys_0
			TArray<TPair<const FString*, const FConcertExportedObject*>> RelatedObjectGroup;
			for (const TPair<const FString*, const FConcertExportedObject*>& ModifiedPair : StandaloneObjects)
			{
				// Group related objects together if some relation can be found between them.
				if (RelatedObjectGroup.Num() == 0 ||
					FPackageName::ObjectPathToSubObjectPath(ModifiedPair.Value->ObjectId.ObjectOuterPathName.ToString()).StartsWith(FPackageName::ObjectPathToSubObjectPath(RelatedObjectGroup[0].Value->ObjectId.ObjectOuterPathName.ToString())))
				{
					RelatedObjectGroup.Emplace(ModifiedPair);
				}
				else if (RelatedObjectGroup.Num() == 1) // End of the group, the object is unrelated to others.
				{
					StoryObjects.Emplace(RelatedObjectGroup[0]); // Produce a story for object.
					RelatedObjectGroup[0] = ModifiedPair;
				}
				else // End of the group, multiple objects are related to each other, but not in a clear parent-child relationship.
				{
					// When multiple sub-objects have the same owner, try selecting the best one for the story.
					StoryObjects.Emplace(SelectModifiedComponent(RelatedObjectGroup));

					// Start a new group.
					RelatedObjectGroup.Reset();
					RelatedObjectGroup.Emplace(ModifiedPair);
				}
			}

			if (RelatedObjectGroup.Num()) // Ensure to deal with any left-over objects.
			{
				StoryObjects.Emplace(SelectModifiedComponent(RelatedObjectGroup));
			}
		}

		// DEBUG: Print the objects selected to create stories.
		//ActivitySummaryUtil::DebugPrintExportedObjects(TEXT("Stories"), StoryObjects);

		// Publish the primary story from the selected objects.
		if (StoryObjects.Num() > 0)
		{
			const TPair<const FString*, const FConcertExportedObject*>& StoryPair = StoryObjects[0];

			FString ObjectOuterPathname = StoryPair.Value->ObjectId.ObjectOuterPathName.ToString();
			FName PackageName(*FPackageName::ObjectPathToPackageName(ObjectOuterPathname));

			if (StoryPair.Value->ObjectData.bIsPendingKill) // Something was deleted?
			{
				FName ObjectDisplayName = ActivitySummaryUtil::GetObjectDisplayName(ObjectOuterPathname, StoryPair.Value->ObjectId.ObjectName);
				FillActivitySummary(EConcertSyncTransactionActivitySummaryType::Deleted, ObjectDisplayName, PackageName, FName(), StoryObjects.Num(), OutActivitySummary);
			}
			else if (StoryPair.Value->ObjectData.bAllowCreate) // Something was created.
			{
				FName ObjectName = StoryPair.Value->ObjectData.NewName.IsNone() ? StoryPair.Value->ObjectId.ObjectName : StoryPair.Value->ObjectData.NewName;
				FName ObjectDisplayName = ActivitySummaryUtil::GetObjectDisplayName(ObjectOuterPathname, ObjectName);
				FillActivitySummary(EConcertSyncTransactionActivitySummaryType::Added, ObjectDisplayName, PackageName, FName(), StoryObjects.Num(), OutActivitySummary);
			}
			else // Something was modified.
			{
				FName ObjectDisplayName = ActivitySummaryUtil::GetObjectDisplayName(ObjectOuterPathname, StoryPair.Value->ObjectId.ObjectName);
				FillActivitySummary(EConcertSyncTransactionActivitySummaryType::Updated, ObjectDisplayName, PackageName, FName(), StoryObjects.Num(), OutActivitySummary);
			}
		}

		// DEBUG: Uncomment to delimit the logs for this execution of the function.
		//UE_LOGF(LogConcert, Display, "===============================END TRANSACTION================================");
	};

	FConcertSyncTransactionActivitySummary ActivitySummary;
	CalculateActivitySummary(ActivitySummary);
	return ActivitySummary;
}

FText FConcertSyncTransactionActivitySummary::CreateDisplayText(const bool InUseRichText) const
{
	FText FormatPattern;
	if (NumActions == 1)
	{
		switch (TransactionSummaryType)
		{
		case EConcertSyncTransactionActivitySummaryType::Added:
			FormatPattern = LOCTEXT("CreateDisplayText_Transaction_AddedOne", "Created {PrimaryObjectName} in {PrimaryPackageName}.");
			break;
		case EConcertSyncTransactionActivitySummaryType::Updated:
			FormatPattern = LOCTEXT("CreateDisplayText_Transaction_UpdatedOne", "Modified {PrimaryObjectName} in {PrimaryPackageName}.");
			break;
		case EConcertSyncTransactionActivitySummaryType::Renamed:
			FormatPattern = LOCTEXT("CreateDisplayText_Transaction_RenamedOne", "Renamed {PrimaryObjectName} to {NewObjectName} in {PrimaryPackageName}.");
			break;
		case EConcertSyncTransactionActivitySummaryType::Deleted:
			FormatPattern = LOCTEXT("CreateDisplayText_Transaction_DeletedOne", "Deleted {PrimaryObjectName} in {PrimaryPackageName}.");
			break;
		default:
			checkf(false, TEXT("Unhandled EConcertSyncTransactionActivitySummaryType in FConcertSyncTransactionActivitySummary!"));
			break;
		}
	}
	else
	{
		switch (TransactionSummaryType)
		{
		case EConcertSyncTransactionActivitySummaryType::Added:
			FormatPattern = LOCTEXT("CreateDisplayText_Transaction_AddedMany", "Created {PrimaryObjectName} in {PrimaryPackageName}, plus {NumOtherActions} other {NumOtherActions}|plural(one=action,other=actions).");
			break;
		case EConcertSyncTransactionActivitySummaryType::Updated:
			FormatPattern = LOCTEXT("CreateDisplayText_Transaction_UpdatedMany", "Modified {PrimaryObjectName} in {PrimaryPackageName}, plus {NumOtherActions} other {NumOtherActions}|plural(one=action,other=actions).");
			break;
		case EConcertSyncTransactionActivitySummaryType::Renamed:
			FormatPattern = LOCTEXT("CreateDisplayText_Transaction_RenamedMany", "Renamed {PrimaryObjectName} to {NewObjectName} in {PrimaryPackageName}, plus {NumOtherActions} other {NumOtherActions}|plural(one=action,other=actions).");
			break;
		case EConcertSyncTransactionActivitySummaryType::Deleted:
			FormatPattern = LOCTEXT("CreateDisplayText_Transaction_DeletedMany", "Deleted {PrimaryObjectName} in {PrimaryPackageName}, plus {NumOtherActions} other {NumOtherActions}|plural(one=action,other=actions).");
			break;
		default:
			checkf(false, TEXT("Unhandled EConcertSyncTransactionActivitySummaryType in FConcertSyncTransactionActivitySummary!"));
			break;
		}
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("PrimaryObjectName"), ActivitySummaryUtil::ToRichTextBold(PrimaryObjectName, InUseRichText));
	Arguments.Add(TEXT("PrimaryPackageName"), ActivitySummaryUtil::ToRichTextBold(PrimaryPackageName, InUseRichText));
	Arguments.Add(TEXT("NewObjectName"), ActivitySummaryUtil::ToRichTextBold(NewObjectName, InUseRichText));
	Arguments.Add(TEXT("NumOtherActions"), NumActions - 1);
	return FText::Format(FormatPattern, Arguments);
}

FText FConcertSyncTransactionActivitySummary::CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const
{
	FText FormatPattern;
	if (NumActions == 1)
	{
		switch (TransactionSummaryType)
		{
		case EConcertSyncTransactionActivitySummaryType::Added:
			FormatPattern = LOCTEXT("CreateDisplayTextForUser_Transaction_AddedOne", "{UserName} created {PrimaryObjectName} in {PrimaryPackageName}.");
			break;
		case EConcertSyncTransactionActivitySummaryType::Updated:
			FormatPattern = LOCTEXT("CreateDisplayTextForUser_Transaction_UpdatedOne", "{UserName} modified {PrimaryObjectName} in {PrimaryPackageName}.");
			break;
		case EConcertSyncTransactionActivitySummaryType::Renamed:
			FormatPattern = LOCTEXT("CreateDisplayTextForUser_Transaction_RenamedOne", "{UserName} renamed {PrimaryObjectName} to {NewObjectName} in {PrimaryPackageName}.");
			break;
		case EConcertSyncTransactionActivitySummaryType::Deleted:
			FormatPattern = LOCTEXT("CreateDisplayTextForUser_Transaction_DeletedOne", "{UserName} deleted {PrimaryObjectName} in {PrimaryPackageName}.");
			break;
		default:
			checkf(false, TEXT("Unhandled EConcertSyncTransactionActivitySummaryType in FConcertSyncTransactionActivitySummary!"));
			break;
		}
	}
	else
	{
		switch (TransactionSummaryType)
		{
		case EConcertSyncTransactionActivitySummaryType::Added:
			FormatPattern = LOCTEXT("CreateDisplayTextForUser_Transaction_AddedMany", "{UserName} created {PrimaryObjectName} in {PrimaryPackageName}, plus {NumOtherActions} other {NumOtherActions}|plural(one=action,other=actions).");
			break;
		case EConcertSyncTransactionActivitySummaryType::Updated:
			FormatPattern = LOCTEXT("CreateDisplayTextForUser_Transaction_UpdatedMany", "{UserName} modified {PrimaryObjectName} in {PrimaryPackageName}, plus {NumOtherActions} other {NumOtherActions}|plural(one=action,other=actions).");
			break;
		case EConcertSyncTransactionActivitySummaryType::Renamed:
			FormatPattern = LOCTEXT("CreateDisplayTextForUser_Transaction_RenamedMany", "{UserName} renamed {PrimaryObjectName} to {NewObjectName} in {PrimaryPackageName}, plus {NumOtherActions} other {NumOtherActions}|plural(one=action,other=actions).");
			break;
		case EConcertSyncTransactionActivitySummaryType::Deleted:
			FormatPattern = LOCTEXT("CreateDisplayTextForUser_Transaction_DeletedMany", "{UserName} deleted {PrimaryObjectName} in {PrimaryPackageName}, plus {NumOtherActions} other {NumOtherActions}|plural(one=action,other=actions).");
			break;
		default:
			checkf(false, TEXT("Unhandled EConcertSyncTransactionActivitySummaryType in FConcertSyncTransactionActivitySummary!"));
			break;
		}
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("UserName"), ActivitySummaryUtil::ToRichTextBold(InUserDisplayName, InUseRichText));
	Arguments.Add(TEXT("PrimaryObjectName"), ActivitySummaryUtil::ToRichTextBold(PrimaryObjectName, InUseRichText));
	Arguments.Add(TEXT("PrimaryPackageName"), ActivitySummaryUtil::ToRichTextBold(PrimaryPackageName, InUseRichText));
	Arguments.Add(TEXT("NewObjectName"), ActivitySummaryUtil::ToRichTextBold(NewObjectName, InUseRichText));
	Arguments.Add(TEXT("NumOtherActions"), NumActions - 1);
	return FText::Format(FormatPattern, Arguments);
}


FConcertSyncPackageActivitySummary FConcertSyncPackageActivitySummary::CreateSummaryForEvent(const FConcertPackageInfo& PackageInfo)
{
	FConcertSyncPackageActivitySummary ActivitySummary;
	ActivitySummary.PackageName = PackageInfo.PackageName;
	ActivitySummary.NewPackageName = PackageInfo.NewPackageName;
	ActivitySummary.PackageUpdateType = PackageInfo.PackageUpdateType;
	ActivitySummary.bAutoSave = PackageInfo.bAutoSave;
	ActivitySummary.bPreSave = PackageInfo.bPreSave;
	return ActivitySummary;
}

FText FConcertSyncPackageActivitySummary::CreateDisplayText(const bool InUseRichText) const
{
	FText FormatPattern;
	switch (PackageUpdateType)
	{
	case EConcertPackageUpdateType::Dummy:
		FormatPattern = LOCTEXT("CreateDisplayText_Package_Dummy", "Discarded changes to package {PackageName}.");
		break;
	case EConcertPackageUpdateType::Added:
		FormatPattern = LOCTEXT("CreateDisplayText_Package_Added", "Added package {PackageName}.");
		break;
	case EConcertPackageUpdateType::Saved:
		if (bPreSave)
		{
			FormatPattern = bAutoSave
				? LOCTEXT("CreateDisplayText_Package_PreAutoSaved", "Captured package {PackageName} original state.")
				: LOCTEXT("CreateDisplayText_Package_PreSaved", "Captured package {PackageName} original state.");
		}
		else
		{
			FormatPattern = bAutoSave
				? LOCTEXT("CreateDisplayText_Package_AutoSaved", "Auto-saved package {PackageName}.") 
				: LOCTEXT("CreateDisplayText_Package_Saved", "Saved package {PackageName}.");
		}
		break;
	case EConcertPackageUpdateType::Renamed:
		FormatPattern = LOCTEXT("CreateDisplayText_Package_Renamed", "Renamed package {PackageName} to {NewPackageName}.");
		break;
	case EConcertPackageUpdateType::Deleted:
		FormatPattern = LOCTEXT("CreateDisplayText_Package_Deleted", "Deleted package {PackageName}.");
		break;
	default:
		checkf(false, TEXT("Unhandled EConcertPackageUpdateType in FConcertSyncPackageActivitySummary!"));
		break;
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("PackageName"), ActivitySummaryUtil::ToRichTextBold(PackageName, InUseRichText));
	Arguments.Add(TEXT("NewPackageName"), ActivitySummaryUtil::ToRichTextBold(NewPackageName, InUseRichText));
	return FText::Format(FormatPattern, Arguments);
}

FText FConcertSyncPackageActivitySummary::CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const
{
	FText FormatPattern;
	switch (PackageUpdateType)
	{
	case EConcertPackageUpdateType::Dummy:
		FormatPattern = LOCTEXT("CreateDisplayTextForUser_Package_Dummy", "{UserName} discarded changes to package {PackageName}.");
		break;
	case EConcertPackageUpdateType::Added:
		FormatPattern = LOCTEXT("CreateDisplayTextForUser_Package_Added", "{UserName} added package {PackageName}.");
		break;
	case EConcertPackageUpdateType::Saved:
		FormatPattern = bAutoSave 
			? LOCTEXT("CreateDisplayTextForUser_Package_AutoSaved", "{UserName} auto-saved package {PackageName}.") 
			: LOCTEXT("CreateDisplayTextForUser_Package_Saved", "{UserName} saved package {PackageName}.");
		break;
	case EConcertPackageUpdateType::Renamed:
		FormatPattern = LOCTEXT("CreateDisplayTextForUser_Package_Renamed", "{UserName} renamed package {PackageName} to {NewPackageName}.");
		break;
	case EConcertPackageUpdateType::Deleted:
		FormatPattern = LOCTEXT("CreateDisplayTextForUser_Package_Deleted", "{UserName} deleted package {PackageName}.");
		break;
	default:
		checkf(false, TEXT("Unhandled EConcertPackageUpdateType in FConcertSyncPackageActivitySummary!"));
		break;
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("UserName"), ActivitySummaryUtil::ToRichTextBold(InUserDisplayName, InUseRichText));
	Arguments.Add(TEXT("PackageName"), ActivitySummaryUtil::ToRichTextBold(PackageName, InUseRichText));
	Arguments.Add(TEXT("NewPackageName"), ActivitySummaryUtil::ToRichTextBold(NewPackageName, InUseRichText));
	return FText::Format(FormatPattern, Arguments);
}

namespace UE::ConcertSyncCore::ActivitySummary::Private
{
	static FText CreateDisplayText_LeaveReplication(const FConcertSyncReplicationActivitySummary& Summary, const bool bInUseRichText)
	{
		FConcertSyncReplicationSummary_LeaveReplication Content;
		const bool bRead = Summary.GetSummaryData(Content);
		if (!bRead)
		{
			return LOCTEXT("CreateDisplayText.LeftActivity.NoData", "No data.");
		}
			
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("NumObjects"), ActivitySummaryUtil::ToRichTextBold(FText::AsNumber(Content.OwnedObjects.Num()), bInUseRichText));
		return FText::Format(LOCTEXT("CreateDisplayText.LeftActivity.Fmt", "Stopped replicating {NumObjects} {NumObjects}|plural(one=object, other=objects)."), Arguments);
	}
	static FText CreateDisplayTextForUser_LeaveReplication(const FConcertSyncReplicationActivitySummary& Summary, const FText InUserDisplayName, const bool InUseRichText)
	{
		FConcertSyncReplicationSummary_LeaveReplication Content;
		const bool bRead = Summary.GetSummaryData(Content);
		if (!bRead)
		{
			return LOCTEXT("CreateDisplayTextForUser.LeftActivity.NoData", "No data.");
		}
			
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("UserName"), ActivitySummaryUtil::ToRichTextBold(InUserDisplayName, InUseRichText));
		Arguments.Add(TEXT("NumObjects"), ActivitySummaryUtil::ToRichTextBold(FText::AsNumber(Content.OwnedObjects.Num()), InUseRichText));
		return FText::Format(LOCTEXT("CreateDisplayTextForUser.LeftActivity.Fmt", "{UserName} stopped replicating {NumObjects} {NumObjects}|plural(one=object, other=objects)."), Arguments);
	}

	static FText SelectText(bool bMuted, bool bUnmuted, const FText& MutedOnly, const FText& UnmutedOnly, const FText& MutedAndUnmuted)
	{
		if (bMuted && bUnmuted)
		{
			return MutedAndUnmuted;
		}

		if (bMuted)
		{
			return MutedOnly;
		}

		if (bUnmuted)
		{
			return UnmutedOnly;
		}

		return LOCTEXT("CreateDisplayText.Mute.None", "No changes");
	}
	
	static FText CreateDisplayText_Mute(const FConcertSyncReplicationActivitySummary& Summary, const bool bInUseRichText)
	{
		FConcertSyncReplicationSummary_Mute Content;
		const bool bRead = Summary.GetSummaryData(Content);
		if (!bRead)
		{
			return LOCTEXT("CreateDisplayText.LeftActivity.NoData", "No data.");
		}
		const FConcertReplication_ChangeMuteState_Request& Request = Content.Request;
			
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("NumMuted"), ActivitySummaryUtil::ToRichTextBold(FText::AsNumber(Request.ObjectsToMute.Num()), bInUseRichText));
		Arguments.Add(TEXT("NumUnmuted"), ActivitySummaryUtil::ToRichTextBold(FText::AsNumber(Request.ObjectsToUnmute.Num()), bInUseRichText));
		
		const FText TextToFormat = SelectText(
			!Request.ObjectsToMute.IsEmpty(),
			!Request.ObjectsToUnmute.IsEmpty(),
			LOCTEXT("CreateDisplayText.MutedOnly.Fmt", "Paused {NumMuted} {NumMuted}|plural(one=object, other=objects)."),
			LOCTEXT("CreateDisplayText.UnmutedOnly.Fmt", "Resumed {NumUnmuted} {NumUnmuted}|plural(one=object, other=objects)."),
			LOCTEXT("CreateDisplayText.MutedAndUnmuted.Fmt", "Paused {NumObjects} and resumed {NumUnmuted} objects.")
			);
		return FText::Format(TextToFormat, Arguments);
	}
	static FText CreateDisplayTextForUser_Mute(const FConcertSyncReplicationActivitySummary& Summary, const FText InUserDisplayName, const bool bInUseRichText)
	{
		FConcertSyncReplicationSummary_Mute Content;
		const bool bRead = Summary.GetSummaryData(Content);
		if (!bRead)
		{
			return LOCTEXT("CreateDisplayText.LeftActivity.NoData", "No data.");
		}
		const FConcertReplication_ChangeMuteState_Request& Request = Content.Request;
			
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("NumMuted"), ActivitySummaryUtil::ToRichTextBold(FText::AsNumber(Request.ObjectsToMute.Num()), bInUseRichText));
		Arguments.Add(TEXT("NumUnmuted"), ActivitySummaryUtil::ToRichTextBold(FText::AsNumber(Request.ObjectsToUnmute.Num()), bInUseRichText));
		Arguments.Add(TEXT("UserName"), ActivitySummaryUtil::ToRichTextBold(InUserDisplayName, bInUseRichText));
		
		const FText TextToFormat = SelectText(
			!Request.ObjectsToMute.IsEmpty(),
			!Request.ObjectsToUnmute.IsEmpty(),
			LOCTEXT("CreateDisplayTextForUser.MutedOnly.Fmt", "{UserName} paused {NumMuted} {NumMuted}|plural(one=object, other=objects)."),
			LOCTEXT("CreateDisplayTextForUser.UnmutedOnly.Fmt", "{UserName} resumed {NumUnmuted} {NumUnmuted}|plural(one=object, other=objects)."),
			LOCTEXT("CreateDisplayTextForUser.MutedAndUnmuted.Fmt", "{UserName} paused {NumObjects} and resumed {NumUnmuted} objects.")
			);
		return FText::Format(TextToFormat, Arguments);
	}
}

FText FConcertSyncReplicationActivitySummary::CreateDisplayText(const bool InUseRichText) const
{
	using namespace UE::ConcertSyncCore::ActivitySummary::Private;
	
	static_assert(static_cast<uint8>(EConcertSyncReplicationActivityType::Count) == 3, "If you added an EConcertSyncReplicationActivityType entry, update this switch");
	switch (ActivityType)
	{
	case EConcertSyncReplicationActivityType::LeaveReplication: return CreateDisplayText_LeaveReplication(*this, InUseRichText);
	case EConcertSyncReplicationActivityType::Mute: return CreateDisplayText_Mute(*this, InUseRichText);
	case EConcertSyncReplicationActivityType::None: [[fallthrough]];
		default: checkNoEntry(); return FConcertSyncActivitySummary::CreateDisplayText(InUseRichText);
	}
}

FText FConcertSyncReplicationActivitySummary::CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const
{
	using namespace UE::ConcertSyncCore::ActivitySummary::Private;
	
	static_assert(static_cast<uint8>(EConcertSyncReplicationActivityType::Count) == 3, "If you added an EConcertSyncReplicationActivityType entry, update this switch");
	switch (ActivityType)
	{
	case EConcertSyncReplicationActivityType::LeaveReplication: return CreateDisplayTextForUser_LeaveReplication(*this, InUserDisplayName, InUseRichText);
	case EConcertSyncReplicationActivityType::Mute: return CreateDisplayTextForUser_Mute(*this, InUserDisplayName, InUseRichText);
	case EConcertSyncReplicationActivityType::None: [[fallthrough]];
	default: checkNoEntry(); return FConcertSyncActivitySummary::CreateDisplayText(InUseRichText);
	}
}

#undef LOCTEXT_NAMESPACE

