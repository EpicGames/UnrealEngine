// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertWorkspaceData.h"
#include "ConcertTransactionEvents.h"
#include "UObject/StructOnScope.h"
#include "ConcertSyncSessionTypes.generated.h"

#define UE_API CONCERTSYNCCORE_API

/** Types of connection events */
UENUM()
enum class EConcertSyncConnectionEventType : uint8
{
	Connected = 0,
	Disconnected,
};

/** Types of lock events */
UENUM()
enum class EConcertSyncLockEventType : uint8
{
	Locked = 0,
	Unlocked,
};

/** Types of activity events */
UENUM()
enum class EConcertSyncActivityEventType : uint8
{
	None = 0,
	Connection,
	Lock,
	Transaction,
	Package,
	Replication,

	// ADD NEW ENTRIES ABOVE 
	Count
};

/** Type of transaction summaries */
UENUM()
enum class EConcertSyncTransactionActivitySummaryType : uint8
{
	Added,
	Updated,
	Renamed,
	Deleted,
};

/** Data for an endpoint in a Concert Sync Session */
USTRUCT()
struct FConcertSyncEndpointData
{
	GENERATED_BODY()

	/** The information about the Concert client connected through this endpoint */
	UPROPERTY()
	FConcertClientInfo ClientInfo;
};

/** ID and data pair for an endpoint in a Concert Sync Session */
USTRUCT()
struct FConcertSyncEndpointIdAndData
{
	GENERATED_BODY()

	/** The ID of the endpoint */
	UPROPERTY()
	FGuid EndpointId;

	/** The data for the endpoint */
	UPROPERTY()
	FConcertSyncEndpointData EndpointData;
};

/** Data for a connection event in a Concert Sync Session */
USTRUCT()
struct FConcertSyncConnectionEvent
{
	GENERATED_BODY()

	/** The type of this connection event */
	UPROPERTY()
	EConcertSyncConnectionEventType ConnectionEventType = EConcertSyncConnectionEventType::Connected;
};

/** Data for a lock event in a Concert Sync Session */
USTRUCT()
struct FConcertSyncLockEvent
{
	GENERATED_BODY()

	/** The type of this lock event */
	UPROPERTY()
	EConcertSyncLockEventType LockEventType = EConcertSyncLockEventType::Locked;

	/** The resources affected by this lock event */
	UPROPERTY()
	TArray<FName> ResourceNames;
};

/** Data for a transaction event in a Concert Sync Session */
USTRUCT()
struct FConcertSyncTransactionEvent
{
	GENERATED_BODY()

	/** The transaction data for this event */
	UPROPERTY()
	FConcertTransactionFinalizedEvent Transaction;
};

/** Data for a package event in a Concert Sync Session */
USTRUCT()
struct FConcertSyncPackageEvent
{
	GENERATED_BODY()

	/** The revision of this package within the session? */
	UPROPERTY()
	int64 PackageRevision = 0;

	/** The package data for this event */
	UPROPERTY()
	FConcertPackage Package;
};

/** Meta data for a package event in a Concert Sync Session. */
USTRUCT()
struct FConcertSyncPackageEventMetaData
{
	GENERATED_BODY()

	/** The revision of this package within the session. */
	UPROPERTY()
	int64 PackageRevision = 0;

	/** Contains information about the package event such as the package name, the event type, if this was triggered by an auto-save, etc. */
	UPROPERTY()
	FConcertPackageInfo PackageInfo;
};

/** Used to stream the package data. */
struct FConcertPackageDataStream
{
	/** The package data, positioned to read the first byte of package data, not necessarily at zero, some unrelated data can be store before. Can be null. */
	FArchive* DataAr = nullptr;

	/** The size of the package data. Can 0 up to several GB large. Does't necessarily extend to the end of the archive, some unrelated data can be stored after.*/
	int64 DataSize = 0;

	/** An array of bytes containing the package data if the data was already available in memory, null otherwise. Can optimize few cases when available. */
	const TArray<uint8>* DataBlob = nullptr;
};

/** Contains a package event where the package data is represented by a stream because it can be very large (several GB) */
struct FConcertSyncPackageEventData
{
	/** The package event meta data.*/
	FConcertSyncPackageEventMetaData MetaData;

	/** The package data. */
	FConcertPackageDataStream PackageDataStream;
};

UENUM(Flags)
enum class EConcertSyncActivityFlags : uint8
{
	None = 0,
	/**
	 * This activity will never be sent to clients by the server.
	 * For all activities a client receives (Flags & EConcertSyncActivityFlags::Muted) == EConcertSyncActivityFlags::None holds. 
	 */
	Muted = 1 << 0,
};
ENUM_CLASS_FLAGS(EConcertSyncActivityFlags)

/** Data for an activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncActivity
{
	GENERATED_BODY()

	/** The ID of the activity */
	UPROPERTY()
	int64 ActivityId = 0;

	/** True if this activity is included for tracking purposes only, and can be ignored when migrating a database */
	UPROPERTY()
	bool bIgnored = false;

	/** Additional information about this activity */
	UPROPERTY()
	EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None;

	/** The ID of the endpoint that produced the activity */
	UPROPERTY()
	FGuid EndpointId;

	/** The time at which the activity was produced (UTC) */
	UPROPERTY()
	FDateTime EventTime = {0};

	/** The type of this activity */
	UPROPERTY()
	EConcertSyncActivityEventType EventType = EConcertSyncActivityEventType::None;

	/** The ID of the event associated with this activity (@see EventType to work out how to resolve this) */
	UPROPERTY()
	int64 EventId = 0;

	/** The minimal summary of the event associated with this activity (@see FConcertSyncActivitySummary) */
	UPROPERTY()
	FConcertSessionSerializedPayload EventSummary{EConcertPayloadSerializationMethod::Cbor};
};

/** Data for a connection activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncConnectionActivity : public FConcertSyncActivity
{
	GENERATED_BODY()

	FConcertSyncConnectionActivity()
	{
		EventType = EConcertSyncActivityEventType::Connection;
	}

	/** The connection event data associated with this activity */
	UPROPERTY()
	FConcertSyncConnectionEvent EventData;
};

/** Data for a lock activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncLockActivity : public FConcertSyncActivity
{
	GENERATED_BODY()

	FConcertSyncLockActivity()
	{
		EventType = EConcertSyncActivityEventType::Lock;
	}

	/** The lock event data associated with this activity */
	UPROPERTY()
	FConcertSyncLockEvent EventData;
};

/** Data for a transaction activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncTransactionActivity : public FConcertSyncActivity
{
	GENERATED_BODY()

	FConcertSyncTransactionActivity()
	{
		EventType = EConcertSyncActivityEventType::Transaction;
	}

	/** The transaction event data associated with this activity */
	UPROPERTY()
	FConcertSyncTransactionEvent EventData;
};

/** Data for a package activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncPackageActivity : public FConcertSyncActivity
{
	GENERATED_BODY()

	FConcertSyncPackageActivity()
	{
		EventType = EConcertSyncActivityEventType::Package;
	}

	/** The package event data associated with this activity */
	UPROPERTY()
	FConcertSyncPackageEvent EventData;
};

/** Base summary for an activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncActivitySummary
{
	GENERATED_BODY()

	virtual ~FConcertSyncActivitySummary() = default;

public:
	/**
	 * Get the display text of the activity summary.
	 * 
	 * @param InUserDisplayName		The display name of the user the activity belongs to, or an empty text to skip the user information.
	 * @param InUseRichText			True if the output should be suitable for consumption as rich-text, or false for plain-text.
	 */
	UE_API FText ToDisplayText(const FText InUserDisplayName, const bool InUseRichText = false) const;

protected:
	UE_API virtual FText CreateDisplayText(const bool InUseRichText) const;
	UE_API virtual FText CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const;
};

/** Summary for a connection activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncConnectionActivitySummary : public FConcertSyncActivitySummary
{
	GENERATED_BODY()

	/** The type of connection event we summarize */
	UPROPERTY()
	EConcertSyncConnectionEventType ConnectionEventType = EConcertSyncConnectionEventType::Connected;

	/** Create this summary from an connection event */
	static UE_API FConcertSyncConnectionActivitySummary CreateSummaryForEvent(const FConcertSyncConnectionEvent& InEvent);

protected:
	//~ FConcertSyncActivitySummary interface
	UE_API virtual FText CreateDisplayText(const bool InUseRichText) const override;
	UE_API virtual FText CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const override;
};

/** Summary for a lock activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncLockActivitySummary : public FConcertSyncActivitySummary
{
	GENERATED_BODY()

	/** The type of lock event we summarize */
	UPROPERTY()
	EConcertSyncLockEventType LockEventType = EConcertSyncLockEventType::Locked;

	/** The primary resource affected by the lock event we summarize */
	UPROPERTY()
	FName PrimaryResourceName;

	/** The primary package affected by the lock event we summarize */
	UPROPERTY()
	FName PrimaryPackageName;

	/** The total number of resources affected by the lock event we summarize */
	UPROPERTY()
	int32 NumResources = 0;

	/** Create this summary from a lock event */
	static UE_API FConcertSyncLockActivitySummary CreateSummaryForEvent(const FConcertSyncLockEvent& InEvent);

protected:
	//~ FConcertSyncActivitySummary interface
	UE_API virtual FText CreateDisplayText(const bool InUseRichText) const override;
	UE_API virtual FText CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const override;
};

/** Summary for a transaction activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncTransactionActivitySummary : public FConcertSyncActivitySummary
{
	GENERATED_BODY()

	/** The type of summary that the transaction event we summarize produced */
	UPROPERTY()
	EConcertSyncTransactionActivitySummaryType TransactionSummaryType = EConcertSyncTransactionActivitySummaryType::Updated;

	/** The title of transaction in the transaction event we summarize */
	UPROPERTY()
	FText TransactionTitle;

	/** The primary object affected by the transaction event we summarize */
	UPROPERTY()
	FName PrimaryObjectName;

	/** The primary package affected by the transaction event we summarize */
	UPROPERTY()
	FName PrimaryPackageName;

	/** The new object name for the event we summarize (if TransactionSummaryType == EConcertSyncTransactionActivitySummaryType::Renamed) */
	UPROPERTY()
	FName NewObjectName;

	/** The total number of actions created by the transaction event we summarize */
	UPROPERTY()
	int32 NumActions = 0;

	/** Create this summary from a transaction event */
	static UE_API FConcertSyncTransactionActivitySummary CreateSummaryForEvent(const FConcertSyncTransactionEvent& InEvent);

protected:
	//~ FConcertSyncActivitySummary interface
	UE_API virtual FText CreateDisplayText(const bool InUseRichText) const override;
	UE_API virtual FText CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const override;
};

/** Summary for a package activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncPackageActivitySummary : public FConcertSyncActivitySummary
{
	GENERATED_BODY()

	/** The package affected by the package event we summarize */
	UPROPERTY()
	FName PackageName;

	/** The new package name for the event we summarize (if PackageUpdateType == EConcertPackageUpdateType::Renamed) */
	UPROPERTY()
	FName NewPackageName;

	/** The type of package update we summarize */
	UPROPERTY()
	EConcertPackageUpdateType PackageUpdateType = EConcertPackageUpdateType::Dummy;

	/** Are we summarizing an auto-save update? */
	UPROPERTY()
	bool bAutoSave = false;

	/** Are we summarizing a pre-save update? */
	UPROPERTY()
	bool bPreSave = false;

	/** Create this summary from a package event */
	static UE_API FConcertSyncPackageActivitySummary CreateSummaryForEvent(const FConcertPackageInfo& InEvent);

protected:
	//~ FConcertSyncActivitySummary interface
	UE_API virtual FText CreateDisplayText(const bool InUseRichText) const override;
	UE_API virtual FText CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const override;
};

struct FConcertSessionActivity
{
	FConcertSessionActivity() = default;

	FConcertSessionActivity(const FConcertSyncActivity& InActivity, const FStructOnScope& InActivitySummary, TUniquePtr<FConcertSessionSerializedPayload> OptionalEventPayload = nullptr)
		: Activity(InActivity)
		, EventPayload(MoveTemp(OptionalEventPayload))
	{
		ActivitySummary.InitializeFromChecked(InActivitySummary);
	}

	FConcertSessionActivity(FConcertSyncActivity&& InActivity, FStructOnScope&& InActivitySummary, TUniquePtr<FConcertSessionSerializedPayload> OptionalEventPayload = nullptr)
		: Activity(MoveTemp(InActivity))
		, EventPayload(MoveTemp(OptionalEventPayload))
	{
		ActivitySummary.InitializeFromChecked(MoveTemp(InActivitySummary));
	}

	/** The generic activity part. */
	FConcertSyncActivity Activity;

	/** Contains the activity summary to display as text. */
	TStructOnScope<FConcertSyncActivitySummary> ActivitySummary;

	/**
	 * The activity event payload usable for activity inspection. Might be null if it was not requested or did not provide insightful information.
	 *   - If the activity type is 'transaction' and EventPayload is not null, it contains a FConcertSyncTransactionEvent with full transaction data.
	 *   - If the activity type is 'package' and EventPayload is not null, it contains a FConcertSyncPackageEvent with the package meta data only.
	 *   - Not set for other activity types (connection/lock).
	 * @see FConcertActivityStream
	 */
	TUniquePtr<FConcertSessionSerializedPayload> EventPayload;
};

#undef UE_API
