// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/FieldPath.h"
#include "UObject/UObjectGlobals.h"
#include "Replication/Data/ReplicationActionEntry.h"
#include "ConcertSyncSettings.generated.h"

USTRUCT()
struct FTransactionClassFilter
{
	GENERATED_BODY()

	/**
	 *	Optional Outer Class that will allow object only if one of their outer match this class.
	 */
	UPROPERTY(config, EditAnywhere, Category="Sync Settings")
	FSoftClassPath ObjectOuterClass;

	/**
	 * Object classes to filter transaction object on.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Sync Settings")
	TArray<FSoftClassPath> ObjectClasses;
};

USTRUCT()
struct FPackageClassFilter
{
	GENERATED_BODY()

	/**
	 *	Optional Class that will filter a specific asset type.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Sync Settings")
	FSoftClassPath AssetClass;

	/**
	 *  Paths on which package of a certain asset type will pass the filter, if no asset is specified all asset in those paths pass the filter
	 */
	UPROPERTY(config, EditAnywhere, Category = "Sync Settings")
	TArray<FString> ContentPaths;
};

UENUM()
enum class EShowPresenceInGame
{
	/** The presence actor is not visible in game mode. */
	NotVisibleInGame,

	/** The presence actor is visible in game only in editor sessions. */
	VisibleInGameOnlyEditor,

	/** The presence actor is always visible in game including -game modes. This is equivalent to Concert.EnablePresenceInGame */
	VisibleInGameAlways
};

UCLASS(MinimalAPI, config=ConcertSyncCore)
class UConcertSyncConfig : public UObject
{
	GENERATED_BODY()

public:
	UConcertSyncConfig()
		: bInteractiveHotReload(false)
		, bShowPresenceInPIE(true)
		, bIncludeAnnotationObjectChanges(true)
		, SnapshotTransactionsPerSecond(30.0f)
	{}

	/** @return The global settings objects. */
	static UConcertSyncConfig* Get() { return GetMutableDefault<UConcertSyncConfig>(); }

	/**
	 * Should we ask before hot-reloading changed packages?
	 * If disabled we will clobber any local changes when reloading packages.
	 */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="Transaction Settings")
	bool bInteractiveHotReload;

	/**
	 * Should we show presence when in PIE?
	 */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="Transaction Settings")
	bool bShowPresenceInPIE;

	/**
	 * Should we show presence when we are in game mode when running the editor.
	 */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="Transaction Settings")
	EShowPresenceInGame ShowPresenceInGame = EShowPresenceInGame::NotVisibleInGame;

	/**
	 * Should we include object changes that have been generated via a transaction annotation
	 * (where possible), or should we send the entire transaction annotation blob instead.
	 */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="Transaction Settings")
	bool bIncludeAnnotationObjectChanges;

	/**
	 * Number of snapshot transactions (eg, moving an object or dragging a slider) that should be sent per-second to other clients.
	 */
	UPROPERTY(config, EditAnywhere, Category="Transaction Settings", meta=(ClampMin="1", UIMin="1"))
	float SnapshotTransactionsPerSecond;

	/**
	 * Array of Transaction class filter.
	 * Only objects that pass those filters will be included in transaction updates.
	 * @note If this is empty, then all class types will send transaction updates.
	 */
	UPROPERTY(config, EditAnywhere, Category="Transaction Settings")
	TArray<FTransactionClassFilter> IncludeObjectClassFilters;

	/**
	 * Array of additional Transaction class filter.
	 * Objects that matches those filters, will prevent the whole transaction from propagation.
	 * @note These filters takes precedence over the IncludeObjectClassFilters
	 */
	UPROPERTY(config, EditAnywhere, Category = "Transaction Settings")
	TArray<FTransactionClassFilter> ExcludeTransactionClassFilters;

	/**
	 * Array of transient class properties that we should send transaction updates for even if usually filtered out.
	 */
	UPROPERTY(config, EditAnywhere, Category="Transaction Settings", meta=(AllowedClasses="/Script/CoreUObject.Property"))
	TArray<TFieldPath<FProperty>> AllowedTransientProperties;

	/**
	 * Array of class properties that we should filtered out.
	 */
	UPROPERTY(config, EditAnywhere, Category="Transaction Settings", meta=(AllowedClasses="/Script/CoreUObject.Property"))
	TArray<TFieldPath<FProperty>> ExcludedProperties;

	/**
	 * Array of property types that we should filtered out.
	 */
	UPROPERTY(config, EditAnywhere, Category="Transaction Settings")
	TArray<FName> ExcludedPropertyTypes;

	/**
	 * Array of package class filter.
	 * Packages that matches those filters, will be excluded from propagating to the server when saved.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Package Settings")
	TArray<FPackageClassFilter> ExcludePackageClassFilters;
	
	/** Defines actions that are to be executed after certain properties are replicated, such as calling PostEditChange, etc. */
	UPROPERTY(Config, EditAnywhere, Category = "Actions")
	TArray<FConcertReplicationActionEntry> PostReplicateActions;
};
