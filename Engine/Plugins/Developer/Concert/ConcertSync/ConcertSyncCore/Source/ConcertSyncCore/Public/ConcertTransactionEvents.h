// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TransactionObjectEvent.h"
#include "IdentifierTable/ConcertIdentifierTableData.h"
#include "ConcertTransactionEvents.generated.h"

UENUM()
enum class ETransactionFilterResult : uint8
{
	/** Include the object in the Concert Transaction */
	IncludeObject,
	/** Filter the object from the Concert Transaction */
	ExcludeObject,
	/** Filter the entire transaction and prevent propagation */
	ExcludeTransaction,
	/** Delegate the filtering decision to the default handlers. */
	UseDefault
};

USTRUCT()
struct FConcertObjectId
{
	GENERATED_BODY()

	FConcertObjectId() = default;

	explicit FConcertObjectId(const UObject* InObject)
		: FConcertObjectId(FTransactionObjectId(InObject), InObject->GetFlags())
	{
	}

	FConcertObjectId(const FTransactionObjectId& InObjectId, const uint32 InObjectFlags)
		: FConcertObjectId(InObjectId.ObjectClassPathName, InObjectId.ObjectPackageName, InObjectId.ObjectName, InObjectId.ObjectOuterPathName, InObjectId.ObjectExternalPackageName, InObjectFlags)
	{
	}

	FConcertObjectId(const FName InObjectClassPathName, const FName InObjectPackageName, const FName InObjectName, const FName InObjectOuterPathName, const FName InObjectExternalPackageName, const uint32 InObjectFlags)
		: ObjectClassPathName(InObjectClassPathName)
		, ObjectPackageName(InObjectPackageName)
		, ObjectName(InObjectName)
		, ObjectOuterPathName(InObjectOuterPathName)
		, ObjectExternalPackageName(InObjectExternalPackageName)
		, ObjectPersistentFlags(InObjectFlags & RF_Load)
	{
	}

	FTransactionObjectId ToTransactionObjectId() const
	{
		FNameBuilder ObjectPathName;
		ObjectOuterPathName.AppendString(ObjectPathName);
		{
			int32 Unused = 0;
			if (ObjectPathName.ToView().FindChar(TEXT('.'), Unused) && !ObjectPathName.ToView().FindChar(SUBOBJECT_DELIMITER_CHAR, Unused))
			{
				ObjectPathName.AppendChar(SUBOBJECT_DELIMITER_CHAR);
			}
			else
			{
				ObjectPathName.AppendChar(TEXT('.'));
			}
		}
		ObjectName.AppendString(ObjectPathName);

		return FTransactionObjectId(ObjectPackageName, ObjectName, ObjectPathName.ToString(), ObjectOuterPathName, ObjectExternalPackageName, ObjectClassPathName);
	}

	bool IsValid() const
	{
		return !ObjectName.IsNone() && !ObjectOuterPathName.IsNone();
	}

	UPROPERTY()
	FName ObjectClassPathName;

	UPROPERTY()
	FName ObjectPackageName;

	UPROPERTY()
	FName ObjectName;

	UPROPERTY()
	FName ObjectOuterPathName;

	UPROPERTY()
	FName ObjectExternalPackageName;

	UPROPERTY()
	uint32 ObjectPersistentFlags = 0;
};

inline bool operator ==(const FConcertObjectId& A, const FConcertObjectId& B)
{
	return A.ObjectClassPathName == B.ObjectClassPathName
		&& A.ObjectOuterPathName == B.ObjectOuterPathName
		&& A.ObjectName == B.ObjectName;
}

inline uint32 GetTypeHash(const FConcertObjectId& ConcertObjectId)
{
	return HashCombine(GetTypeHash(ConcertObjectId.ObjectName), GetTypeHash(ConcertObjectId.ObjectOuterPathName));
}

USTRUCT()
struct FConcertSerializedObjectData
{
	GENERATED_BODY()

	UPROPERTY()
	bool bAllowCreate = false;

	UPROPERTY()
	bool bResetExisting = false;

	UPROPERTY()
	bool bIsPendingKill = false;

	UPROPERTY()
	FName NewPackageName;

	UPROPERTY()
	FName NewName;

	UPROPERTY()
	FName NewOuterPathName;

	UPROPERTY()
	FName NewExternalPackageName;

	UPROPERTY()
	FSoftObjectPath SourceObject;

	UPROPERTY()
	TArray<uint8> SerializedData;
};

USTRUCT()
struct FConcertSerializedPropertyData
{
	GENERATED_BODY()

	UPROPERTY()
	FName PropertyName;

	UPROPERTY()
	TArray<uint8> SerializedData;
};

USTRUCT()
struct FConcertExportedObject
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertObjectId ObjectId;

	UPROPERTY()
	int32 ObjectPathDepth = 0;

	UPROPERTY()
	bool bHasLevelInstanceObject = false;

	UPROPERTY()
	FConcertSerializedObjectData ObjectData;

	UPROPERTY()
	TArray<FConcertSerializedPropertyData> PropertyDatas;

	UPROPERTY()
	TArray<uint8> SerializedAnnotationData;
};

USTRUCT()
struct FConcertTransactionEventBase
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid TransactionId;

	UPROPERTY()
	FGuid OperationId;

	UPROPERTY()
	FGuid TransactionEndpointId;

	UPROPERTY()
	uint8 TransactionUpdateIndex = 0;

	UPROPERTY()
	int32 VersionIndex = INDEX_NONE;

	UPROPERTY()
	TArray<FName> ModifiedPackages;

	UPROPERTY()
	FConcertObjectId PrimaryObjectId;

	UPROPERTY()
	TArray<FConcertExportedObject> ExportedObjects;
};

USTRUCT()
struct FConcertTransactionFinalizedEvent : public FConcertTransactionEventBase
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertLocalIdentifierState LocalIdentifierState;

	UPROPERTY()
	FText Title;
};

USTRUCT()
struct FConcertTransactionSnapshotEvent : public FConcertTransactionEventBase
{
	GENERATED_BODY()
};

USTRUCT()
struct FConcertTransactionRejectedEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid TransactionId;
};
