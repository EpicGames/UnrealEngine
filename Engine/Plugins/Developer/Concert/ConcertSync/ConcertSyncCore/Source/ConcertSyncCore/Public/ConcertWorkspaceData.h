// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "ConcertWorkspaceData.generated.h"

UENUM()
enum class EConcertPackageUpdateType : uint8
{
	/** A dummy update, typically used to fence some transactions as no longer relevant */
	Dummy,
	/** This package has been added, but not yet saved */
	Added,
	/** This package has been saved */
	Saved,
	/** This package has been renamed (leaving a redirector) */
	Renamed,
	/** This package has been deleted */
	Deleted,

	Count
};

USTRUCT()
struct FConcertPackageInfo
{
	GENERATED_BODY()

	/** The name of the package */
	UPROPERTY()
	FName PackageName;

	/** The new name of the package (if PackageUpdateType == EConcertPackageUpdateType::Renamed) */
	UPROPERTY()
	FName NewPackageName;

	/** The class of the asset contained in this package. */
	UPROPERTY()
	FString AssetClass;

	/** The extension of the package file on disk (eg, .umap or .uasset) */
	UPROPERTY()
	FString PackageFileExtension;

	/** What kind of package update is this? */
	UPROPERTY()
	EConcertPackageUpdateType PackageUpdateType = EConcertPackageUpdateType::Dummy;

	/** What was the max transaction event ID when this update was made? (to discard older transactions that applied to this package) */
	UPROPERTY()
	int64 TransactionEventIdAtSave = 0;

	/** Was this update caused by a pre-save? */
	UPROPERTY()
	bool bPreSave = false;

	/** Was this update caused by an auto-save? */
	UPROPERTY()
	bool bAutoSave = false;

	/**
	 * Can we skip hot reload on this package? Hot reload is required to update changes to a loaded asset to a newer versoin
	 * modified in a multi-user session.  Take recordings are packages that do not need to go through hot reload as they have no live assets.
	 * This flag can be set to true to indicate to the replay system the package can bypass hot reload.
	 *
	 * @see IConcertClientClientPackageBridge::RegisterPackageHotReloadHint
	 */
	UPROPERTY()
	bool bCanSkipHotReload = false;
};

USTRUCT()
struct FConcertPackage
{
	GENERATED_BODY()

	/** Contains information about the package event such as the package name, the event type, if this was triggered by an auto-save, etc. */
	UPROPERTY()
	FConcertPackageInfo Info;

	/** Contains the package data, unless the package size was too large (according to the hard limit or a configuration). */
	UPROPERTY()
	FConcertByteArray PackageData;

	/** A link to a file containing the data if the package size was too large to be directly embedded. The package data needs to be retrieved using IConcertFileSharingService interface.*/
	UPROPERTY()
	FString FileId;

	/** Whether some package data is embedded or linked. */
	bool HasPackageData() const
	{
		return !FileId.IsEmpty() || PackageData.Bytes.Num() != 0;
	}

	/** Whether the package data should be embedded in 'PackageData'. Larger packages could still be embedded as byte array if file sharing is not enabled. */
	static bool ShouldEmbedPackageDataAsByteArray(uint64 PackageDataSize)
	{
		return PackageDataSize <= 2 * 1024 * 1024; // All package data below this size should be embedded in 'PackageData' byte array. This is a reasonable and efficient limit.
	}

	/** Whether the package data can safely be embedded in 'PackageData' without overflow (ex 3GB overflows int32 type). Not necessarily reasonable, but safe. */
	static bool CanEmbedPackageDataAsByteArray(uint64 PackageDataSize)
	{
		return PackageDataSize <= GetMaxPackageDataSizeEmbeddableAsByteArray();
	}

	/** The maximum package data size that can be safely embedded in 'PackageData'. (Not necessarily reasonable, but safe). */
	static constexpr uint64 GetMaxPackageDataSizeEmbeddableAsByteArray()
	{
		// The hard limit may not be safe by itself (nor reasonable) if the structure is serialized into another TArray<> with other data. Make it more reasonable, use only x% of the range.
		return (static_cast<uint64>(TNumericLimits<decltype(PackageData.Bytes)::SizeType>::Max()) * 8) / 10;
	}
};
