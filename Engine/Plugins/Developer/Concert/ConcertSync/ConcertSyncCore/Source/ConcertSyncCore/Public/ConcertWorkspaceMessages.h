// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncSessionTypes.h"
#include "ConcertWorkspaceMessages.generated.h"

USTRUCT()
struct FConcertWorkspaceSyncEventBase
{
	GENERATED_BODY()

	UPROPERTY()
	int32 NumRemainingSyncEvents = 0;
};

USTRUCT()
struct FConcertWorkspaceSyncEndpointEvent : public FConcertWorkspaceSyncEventBase
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertSyncEndpointIdAndData Endpoint;
};

USTRUCT()
struct FConcertWorkspaceSyncActivityEvent : public FConcertWorkspaceSyncEventBase
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertSessionSerializedPayload Activity;
};

USTRUCT()
struct FConcertWorkspaceSyncLockEvent : public FConcertWorkspaceSyncEventBase
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName, FGuid> LockedResources;
};

USTRUCT()
struct FConcertWorkspaceSyncRequestedEvent
{
	GENERATED_BODY()

	/** The ID of the first activity to sync */
	UPROPERTY()
	int64 FirstActivityIdToSync = 1;

	/** The ID of the last activity to sync (ignored if bEnableLiveSync is true) */
	UPROPERTY()
	int64 LastActivityIdToSync = MAX_int64;

	/** True if the server workspace should be live-synced to this client as new activity is added, or false if syncing should only happen in response to these sync request events */
	UPROPERTY()
	bool bEnableLiveSync = true;
};

USTRUCT()
struct FConcertWorkspaceSyncCompletedEvent
{
	GENERATED_BODY()
};

/** An event emitted by a client after its workspace has been completely synced and finalized. All transactions are posted and packages have been loaded. */
USTRUCT()
struct FConcertWorkspaceSyncAndFinalizeCompletedEvent
{
	GENERATED_BODY()
};

/** Request to sync an event that was partially synced on the client but for which the full data is required for inspection. FConcertSyncEventResponse is the corresponding response. */
USTRUCT()
struct FConcertSyncEventRequest
{
	GENERATED_BODY()

	/** The type of event to sync. Only Package and Transaction event types are supported. */
	UPROPERTY()
	EConcertSyncActivityEventType EventType = EConcertSyncActivityEventType::None;

	/** The ID of the event to sync. */
	UPROPERTY()
	int64 EventId = 0;
};

/** Response to a FConcertSyncEventRequest request. */
USTRUCT()
struct FConcertSyncEventResponse
{
	GENERATED_BODY()

	/** The payload contains the event corresponding to the requested event type like FConcertSyncTransactionEvent/FConcertSyncPackageEvent or an empty payload if the request failed. */
	UPROPERTY()
	FConcertSessionSerializedPayload Event;
};

USTRUCT()
struct FConcertServerLogging
{
	GENERATED_BODY()

	UPROPERTY()
	bool bLoggingEnabled = false;
};

/** Sent to let the receiver know something is coming. For now only sent from client to server. */
USTRUCT()
struct FConcertPackageTransmissionStartEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid TransmissionId;
	
	UPROPERTY()
	FConcertPackageInfo PackageInfo;

	UPROPERTY()
	uint64 PackageNumBytes = 0;
};


USTRUCT()
struct FConcertPackageUpdateEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid TransmissionId;
	
	UPROPERTY()
	FConcertPackage Package;
};

USTRUCT()
struct FConcertPackageRejectedEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FName PackageName;
};

UENUM()
enum class EConcertResourceLockType : uint8
{
	None,
	Lock,
	Unlock,
};

USTRUCT()
struct FConcertResourceLockEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ClientId;

	UPROPERTY()
	TArray<FName> ResourceNames;

	UPROPERTY()
	EConcertResourceLockType LockType = EConcertResourceLockType::None;
};

USTRUCT()
struct FConcertResourceLockRequest
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ClientId;

	UPROPERTY()
	TArray<FName> ResourceNames;

	UPROPERTY()
	EConcertResourceLockType LockType = EConcertResourceLockType::None;;
};

USTRUCT()
struct FConcertResourceLockResponse
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName, FGuid> FailedResources;

	UPROPERTY()
	EConcertResourceLockType LockType = EConcertResourceLockType::None;;
};

UENUM()
enum class EConcertPlaySessionEventType : uint8
{
	None,
	BeginPlay,
	SwitchPlay,
	EndPlay,
};

USTRUCT()
struct FConcertPlaySessionEvent
{
	GENERATED_BODY()

	UPROPERTY()
	EConcertPlaySessionEventType EventType = EConcertPlaySessionEventType::None;

	UPROPERTY()
	FGuid PlayEndpointId;

	UPROPERTY()
	FName PlayPackageName;

	UPROPERTY()
	bool bIsSimulating = false;
};

/**
 * Sets the specified client 'ignore on restore' state for further activities. The 'ignored' flag can be raised to mark a series of activities as 'should not be restored'.
 * @note This can be used to record and monitor session activities for inspection purpose, for example allowing disaster recovery to record what
 *       happens in a multi-user session without restoring such activities in case of crash (because they occurred in a transient sandbox).
 */
USTRUCT()
struct FConcertIgnoreActivityStateChangedEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid EndpointId;

	UPROPERTY()
	bool bIgnore = false;
};
