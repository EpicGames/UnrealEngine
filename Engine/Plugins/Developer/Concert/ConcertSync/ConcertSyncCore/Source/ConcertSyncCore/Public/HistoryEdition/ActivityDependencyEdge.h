// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActivityGraphIDs.h"

namespace UE::ConcertSyncCore
{
	class FActivityNode;
	class FActivityDependencyGraph;
	
	/** Describes why there is a dependency */
	enum class EActivityDependencyReason
	{
		/**
		 * The depended-on activity created the package that this activity operates on.
		 * 
		 * Example: 1: Create level > 2: Add actor
		 * 2 cannot take place without 1.
		 */
		PackageCreation,

		/**
		 * The depended-on activity deleted a package with the same path name of the package this activity created.
		 * 
		 * Example: 1: Delete level Game/Foo 2: Create new level Game/Foo.
		 * 2 cannot take place without 1.
		 */
		PackageRemoval,

		/**
		 * The depended-on activity renamed a package that is used by this activity.
		 * 
		 * Example: 1: Create level 'Foo' > 2: Rename 'Foo' to 'Bar' > 3: Edit actor
		 * 3 depends on 2.
		 * 
		 * Example: 1: Create level 'Foo' > 2: Rename 'Foo' to 'Bar' > 3: Create level 'Foo'
		 * 3 depends on 1.
		 */
		PackageRename,

		/**
		 * The depended-on activity edits the package that is required by this activity.
		 *
		 * Example: 1: Move actor 2: Save level
		 * 2 depends on 1 (because the save operation would save different data if 1 was not performed).
		 */
		PackageEdited,

		/**
		 * The depended-on activity changes the state of a package that is depended on by a package this activity modifies.
		 * 
		 * Example: 1: Edit actor to reference data asset 2: Edit data asset 3: Edit actor, triggering the construction script.
		 * 3 executes the construction script which may execute differently depending on the data changed in 2.
		 */
		EditAfterPreviousPackageEdit,

		/**
		 * The depended-on activity created a subobject that this activity operates on.
		 * 
		 * Example: 1: Add actor > 2. Edit actor
		 * 2 cannot take place without 1.
		 */
		SubobjectCreation,

		/**
		 * The depended-on activity removed a subobject that this activity requires to be dead.
		 * 
		 * Example: 1: Remove actor > 2. Create actor with same name
		 * 2 cannot take place without 1.
		 */
		SubobjectRemoval,

		// ADD BEFORE THIS ENTRY
		Count
	};
	
	FString LexToString(EActivityDependencyReason Reason);

	enum class EDependencyStrength
	{
		/**
		 * An activity A has a hard dependency to activity B when A cannot exist in an activity history without B.
		 * Example: You create a level and add an actor to it. You cannot create the actor without the level.
		 */
		HardDependency,
		
		/**
		 * An activity A has a possible dependency to activity B when A could possibly exist without B but we cannot rule out
		 * that B does not affect A.
		 * Example: You modify an actor twice each time triggering the construction script
		 */
		PossibleDependency,
	};


	/** Describes a dependency from one activity to an earlier activity. */
	class FActivityDependencyEdge
	{
	public:
		
		FActivityDependencyEdge(FActivityNodeID DependedOnActivityNode, EActivityDependencyReason DependencyReason, EDependencyStrength DependencyStrength)
			: Parent(DependedOnActivityNode)
			, DependencyReason(DependencyReason)
			, DependencyStrength(DependencyStrength)
		{}
		
		FActivityNodeID GetDependedOnNodeID() const { return Parent; }
		EActivityDependencyReason GetDependencyReason() const { return DependencyReason; }
		EDependencyStrength GetDependencyStrength() const { return DependencyStrength; }

		friend bool operator==(const FActivityDependencyEdge& Left, const FActivityDependencyEdge& Right)
		{
			return Left.Parent == Right.Parent
				&& Left.DependencyReason == Right.DependencyReason
				&& Left.DependencyStrength == Right.DependencyStrength;
		}
		friend bool operator!=(const FActivityDependencyEdge& Left, const FActivityDependencyEdge& Right)
		{
			return !(Left == Right);
		}

	private:
		
		/** The node that an activity depends on */
		const FActivityNodeID Parent;

		/** Why this dependency exists */
		const EActivityDependencyReason DependencyReason;

		/** How much the activity is depended on */
		const EDependencyStrength DependencyStrength;
	};
}
