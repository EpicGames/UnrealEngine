// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActivityNode.h"

#define UE_API CONCERTSYNCCORE_API

template <typename FuncType> class TFunctionRef;
template <typename OptionalType> struct TOptional;

namespace UE::ConcertSyncCore { class FActivityDependencyEdge; }
namespace UE::ConcertSyncCore { struct FActivityNodeID; }

namespace UE::ConcertSyncCore
{
	/**
	 * The graph models activities as nodes and dependencies to activities as edges.
	 * Activities can only affect each other from left-to-right: an earlier activity affects a later activity.
	 * 
	 * We say an activity A has a hard dependency to activity B when A cannot exist in an activity history without B.
	 * We say an activity A has a possible dependency to activity B when A may exist with B but we cannot rule out
	 * that B does not affect A. Example: You modify an actor twice each time triggering the construction script.
	 * See EDependencyType.
	 *
	 * A node corresponds to an activity. A root node has depends on no other activity.
	 * An edge from a A to B means that B depends on A.
	 * 
	 * Dependencies do not necessarily need to be transitive: 1. Remove package Foo 2. Create package Foo 3. Edit Foo
	 * The existence of 3 can requires that either
	 *  - 1 and 2 both exist
	 *  - 1 and 2 are deleted (among other details, e.g. that the underlying assets must have equal classes)
	 *      
	 * Example:
	 *	1: Create level named Foo
	 *	2: Create level named Bar
	 *	3: Add actor named 'Cube' to level Foo
	 *	4. Add actor named 'Sphere" to level Foo
	 *	6: Edit 'Cube'.
	 * Resulting graph:
	 *			1		2
	 *		   / \
	 *		  3	  4
	 *		 /
	 *		5
	 *
	 * Example:
	 *  1. Create data asset A
	 *  2. Edit and save A (remember: transactions do not happen for all .uasset only for .umaps)
	 *  3. Create actor in level
	 *  4. Edit actor to reference data assets A
	 *  5. Edit actor whose construction script reads data from A
	 * Resulting Graph:
	 *     1
	 *	   I
	 *     2   3
	 *      \ / 
	 *       4
	 *       I
	 *       5
	 */
	class FActivityDependencyGraph
	{
	public:
		
		/** Adds a node. Invalidates the results of GetNodeById. */
		UE_API FActivityNodeID AddActivity(int64 ActivityIndex, EActivityNodeFlags NodeFlags = EActivityNodeFlags::None);
		
		/** Adds a dependency from an existing node to another existing node */
		UE_API bool AddDependency(FActivityNodeID From, FActivityDependencyEdge To);
		
		/** Tries to find a corresponding node for an activity ID. */
		UE_API TOptional<FActivityNodeID> FindNodeByActivity(int64 ActivityID) const;

		/** Gets the node identified by the given ID, if it is valid. Only valid as long as no new node is added.*/
		UE_API const FActivityNode& GetNodeById(FActivityNodeID ID) const;

		UE_API void ForEachNode(TFunctionRef<void(const FActivityNode&)> ConsumerFunc) const;
		
	private:

		TArray<FActivityNode> Nodes;
		
		UE_API FActivityNode& GetNodeByIdInternal(FActivityNodeID ID);
	};
}

#undef UE_API
