// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ConcertSyncCore
{
	/** Enforces type-safe usage of IDs by avoiding implicit conversions. */
	struct FActivityNodeID
	{
		size_t ID;

		FActivityNodeID() = default;
		
		template<typename T>
		explicit FActivityNodeID(T&& ID)
			: ID(Forward<T>(ID))
		{}

		/** Conversion back to to size_t is not a common programmer mistake */
		explicit operator size_t() const { return ID; }

		friend bool operator==(const FActivityNodeID& Left, const FActivityNodeID& Right)
		{
			return Left.ID == Right.ID;
		}

		friend bool operator!=(const FActivityNodeID& Left, const FActivityNodeID& Right)
		{
			return !(Left == Right);
		}

		FORCEINLINE friend uint32 GetTypeHash(const FActivityNodeID& NodeID)
		{
#if PLATFORM_MAC
			// Help Mac compiler correctly resolve the candidate function
			return ::GetTypeHash(static_cast<uint64>(NodeID.ID));
#else
			return ::GetTypeHash(NodeID.ID);
#endif
		}
	};
}
