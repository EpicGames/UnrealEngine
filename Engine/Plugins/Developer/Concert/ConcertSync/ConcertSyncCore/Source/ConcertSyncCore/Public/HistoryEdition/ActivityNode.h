// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActivityDependencyEdge.h"
#include "Containers/Array.h"
#include "HistoryEdition/ActivityGraphIDs.h"
#include "Misc/Optional.h" // IWYU pragma: keep

#define UE_API CONCERTSYNCCORE_API

struct FScriptContainerElement;

namespace UE::ConcertSyncCore
{
	class FActivityDependencyGraph;

	enum class EActivityNodeFlags
	{
		None			= 0x00,

		/** This activity renames a package */
		RenameActivity	= 0x01
	};
	ENUM_CLASS_FLAGS(EActivityNodeFlags)

	/** A node corresponds to an activity and can depend on preceding activity*/
	class FActivityNode
	{
		friend FActivityDependencyGraph;
	public:
		
		FActivityNode(int64 ActivityId, FActivityNodeID NodeIndex, EActivityNodeFlags NodeFlags)
			: ActivityId(ActivityId)
			, NodeIndex(NodeIndex)
			, NodeFlags(NodeFlags)
		{}

		bool HasAnyDependency() const { return Dependencies.Num() > 0; }
		UE_API bool DependsOnNode(FActivityNodeID NodeId, TOptional<EActivityDependencyReason> WithReason = {}, TOptional<EDependencyStrength> WithStrength = {}) const; 
		UE_API bool DependsOnActivity(int64 InActivityId, const FActivityDependencyGraph& Graph, TOptional<EActivityDependencyReason> WithReason = {}, TOptional<EDependencyStrength> WithStrength = {}) const;
		
		bool AffectsAnyActivity() const { return GetAffectedChildren().Num() > 0; }
		bool AffectsNode(FActivityNodeID NodeID) const { return AffectedChildren.Contains(NodeID); }
		UE_API bool AffectsActivity(int64 InActivityId, const FActivityDependencyGraph& Graph) const;
		
		int64 GetActivityId() const { return ActivityId; }
		FActivityNodeID GetNodeIndex() const { return NodeIndex; }
		EActivityNodeFlags GetNodeFlags() const { return NodeFlags; }
		const TArray<FActivityDependencyEdge>& GetDependencies() const { return Dependencies; }
		const TArray<FActivityNodeID>& GetAffectedChildren() const { return AffectedChildren; }
	
	private:
	
		/** The activity this node corresponds to */
		const int64 ActivityId;

		/** Index in parent tree's node array (graph uses TArray to avoid bad memory footprint) */
		const FActivityNodeID NodeIndex;

		/** Additional info about this activity node */
		const EActivityNodeFlags NodeFlags;

		/** This node's parents. Unset implies this is the root node. */
		TArray<FActivityDependencyEdge> Dependencies;

		/**
		 * This node's children. Children have dependencies to this node.
		 * Activities can only depend on activities that precede them.
		 * Since activity IDs are historic, every child's activity ID > ActivityId.
		 */
		TArray<FActivityNodeID> AffectedChildren;
	};
}

#undef UE_API
