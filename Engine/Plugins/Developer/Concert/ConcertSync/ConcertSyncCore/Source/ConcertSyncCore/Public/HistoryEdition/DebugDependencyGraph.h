// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Misc/EnumClassFlags.h"

class FString;
namespace UE::ConcertSyncCore { class FActivityDependencyEdge; }
namespace UE::ConcertSyncCore { class FActivityDependencyGraph; }
namespace UE::ConcertSyncCore { struct FActivityNodeID; }
template <typename FuncType> class TFunctionRef;

class FConcertSyncSessionDatabase;

namespace UE::ConcertSyncCore::Graphviz
{
	/**
	 * Graphviz syntax:
	 *
	 *	strict digraph {
	 * 		"start node name" [shape="ellipse" style="filled" fillcolor="#1f77b4"]
	 *		"end node name" [shape="polygon" style="filled" fillcolor="#ff7f0e"]
	 *		"start node name" -> "end node name" [fillcolor="#a6cee3" color="#1f78b4" label="Edge label"]
	 *	}	
	 */

	using FGraphStringBuilder = TStringBuilder<512>;
	using FStringifyNode = TFunctionRef<void(FGraphStringBuilder& WriteTo, FActivityNodeID ToStringify)>;
	using FStringifyEdge = TFunctionRef<void(FGraphStringBuilder& WriteTo, FActivityNodeID EdgeStart, const FActivityDependencyEdge& ToStringify)>;

	/** Name of node without quotes, e.g. TEXT("start node name") (see above) */
	using FMakeNodeTitle = FStringifyNode;
	/** Style of the node which is printed right after the name, e.g. TEXT("[shape="ellipse" style="filled" fillcolor="#1f77b4"]") (see above) */
	using FGetNodeStyle = FStringifyNode;
	/** Style of the edge which is printed right after an edge, e.g. TEXT("[fillcolor="#a6cee3" color="#1f78b4" label="Edge label"]") (see above) */
	using FGetEdgeStyle = FStringifyEdge;

	enum class ENodeTitleFlags
	{
		None = 0x00,
		/** Use FConcertSyncActivitySummary::ToDisplayText */
		Summary = 0x01,
		/** Append activity ID */
		ActivityID = 0x02,
		
		Default = Summary | ActivityID
	};
	ENUM_CLASS_FLAGS(ENodeTitleFlags)
	
	/** Prints the title of the activity */
	CONCERTSYNCCORE_API void MakeNodeTitle(FGraphStringBuilder& WriteTo, FActivityNodeID ToStringify, const FActivityDependencyGraph& Graph, const FConcertSyncSessionDatabase& SessionDatabase, ENodeTitleFlags NodeTitleFlags = ENodeTitleFlags::Default);
	/** Changes colour depending on activity type*/
	CONCERTSYNCCORE_API void GetNodeStyle(FGraphStringBuilder& WriteTo, FActivityNodeID ToStringify, const FActivityDependencyGraph& Graph, const FConcertSyncSessionDatabase& SessionDatabase);
	/** Prints dependency reason and changes the arrow icon depending on dependency strength (half arrow for PossibleDependency, full arrow for HardDependency). */
	CONCERTSYNCCORE_API void GetEdgeStyle(FGraphStringBuilder& WriteTo, const FActivityDependencyEdge& ToStringify);

	
	/** Calls ExportToGraphviz with the default formatting functions defined above. */
	CONCERTSYNCCORE_API FString ExportToGraphviz(const FActivityDependencyGraph& Graph, const FConcertSyncSessionDatabase& SessionDatabase, ENodeTitleFlags NodeTitleFlags = ENodeTitleFlags::Default);
	/**
	 * Exports the graph in a syntax that Graphviz (see:http://www.graphviz.org/) understands.
	 * 
	 * Several online editors exist and can be found by searching for "graphviz online", e.g.
	 *	- https://dreampuf.github.io/GraphvizOnline/
	 *	- http://magjac.com/graphviz-visual-editor/
	 */
	CONCERTSYNCCORE_API FString ExportToGraphviz(const FActivityDependencyGraph& Graph, FMakeNodeTitle MakeNodeTitleFunc, FGetNodeStyle GetNodeStyleFunc, FGetEdgeStyle GetEdgeStyleFunc);
}
