// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::ConcertSyncCore { class FActivityDependencyGraph; }

class FConcertSyncSessionDatabase;

namespace UE::ConcertSyncCore
{
	/**
	 * Builds a dependency graph from the given session database.
	 */
	CONCERTSYNCCORE_API FActivityDependencyGraph BuildDependencyGraphFrom(const FConcertSyncSessionDatabase& SessionDatabase);
}
