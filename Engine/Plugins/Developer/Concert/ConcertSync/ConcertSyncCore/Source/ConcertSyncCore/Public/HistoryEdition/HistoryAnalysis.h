// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"

namespace UE::ConcertSyncCore { class FActivityDependencyGraph; }

class FConcertSyncSessionDatabase;

namespace UE::ConcertSyncCore
{
	/** Describes the activities that must be considered when deleting an activity */
	struct FHistoryAnalysisResult
	{
		/** The activities that must be removed. */
		TSet<FActivityID> HardDependencies;
		/**
		 * The activities may want to be removed. It's not certain that they are affected (but it should be safe to keep them in).
		 * This will not contain any elements in HardDependencies.
		 */
		TSet<FActivityID> PossibleDependencies;

		FHistoryAnalysisResult(TSet<FActivityID> HardDependencies = {}, TSet<FActivityID> PossibleDependencies = {})
			: HardDependencies(MoveTemp(HardDependencies))
			, PossibleDependencies(MoveTemp(PossibleDependencies))
		{}
	};

	/** Utility function for one-off operations: just computes the dependency graph before calling AnalyseActivityDeletion. */
	CONCERTSYNCCORE_API FHistoryAnalysisResult AnalyseActivityDependencies_TopDown(const TSet<FActivityID>& ActivitiesToDelete, const FConcertSyncSessionDatabase& Database, bool bAddActivitiesToDelete = false);

	/**
	 * Given a set of activities to be edited (e.g. deleted or muted), returns which activities 1. must be and 2. may want to be considered in addition.
	 * This algorithm proceeds top-down meaning it will start at ActivitiesToEdit and recursively process child nodes that depend on it.
	 * This approach is good for finding activities that depends on a set of activities, e.g. for deleting or muting ActivitiesToEdit.
	 * 
	 * @param ActivitiesToEdit The activities whose children to search
	 * @param DependencyGraph The graph encoding the activity dependencies
	 * @param bAddEditedAsHardDependencies Whether to add ActivitiesToEdit to the result's HardDependencies
	 * @return The activities to consider as well if ActivitiesToEdit are edited; ActivitiesToEdit is included in HardDependencies if bAddActivitiesToDelete == true.
	 */
	CONCERTSYNCCORE_API FHistoryAnalysisResult AnalyseActivityDependencies_TopDown(const TSet<FActivityID>& ActivitiesToEdit, const FActivityDependencyGraph& DependencyGraph, bool bAddEditedAsHardDependencies = false);

	/**
	 * Given a set of activities to be edited (e.g. to be unmuted), returns which activities 1. must be and 2. may want to be considered in addition.
	 * This algorithm proceeds bottom-up meaning it will start at ActivitiesToEdit and recursively process parent nodes it depends on.
	 * This approach is good for finding the activities that are depended on by set of activities, e.g. unmuting ActivitiesToEdit.
	 * 
	 * @param ActivitiesToEdit The activities whose parents to search
	 * @param DependencyGraph The graph encoding the activity dependencies
	 * @param bAddEditedAsHardDependencies Whether to add ActivitiesToEdit to the result's HardDependencies
	 * @return The activities to consider as well if ActivitiesToEdit are edited; ActivitiesToEdit is included in HardDependencies if bAddActivitiesToDelete == true.
	 */
	CONCERTSYNCCORE_API FHistoryAnalysisResult AnalyseActivityDependencies_BottomUp(const TSet<FActivityID>& ActivitiesToEdit, const FActivityDependencyGraph& DependencyGraph, bool bAddEditedAsHardDependencies = false);
}
