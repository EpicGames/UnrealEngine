// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"

namespace UE::ConcertSyncCore { struct FHistoryAnalysisResult; }

class IConcertServer;
class IConcertSyncServer;

namespace UE::ConcertSyncCore
{
	struct FOperationErrorResult
	{
		TOptional<FText> ErrorMessage;

		explicit FOperationErrorResult(TOptional<FText> ErrorMessage = {})
			: ErrorMessage(ErrorMessage)
		{}

		static FOperationErrorResult MakeSuccess() { return FOperationErrorResult(); }
		static FOperationErrorResult MakeError(FText Error) { return FOperationErrorResult(MoveTemp(Error)); }

		bool WasSuccessful() const { return !ErrorMessage.IsSet(); }
		bool HadError() const { return !WasSuccessful(); }
	};

	/** Utility functions that converts FHistoryDeletionRequirements into a single TSet. */
	CONCERTSYNCCORE_API TSet<FActivityID> CombineRequirements(const FHistoryAnalysisResult& ToDelete);
}
