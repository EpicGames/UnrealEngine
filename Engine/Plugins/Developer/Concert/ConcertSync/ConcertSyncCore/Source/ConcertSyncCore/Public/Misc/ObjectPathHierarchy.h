// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EBreakBehavior.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

#define UE_API CONCERTSYNCCORE_API

namespace UE::ConcertSyncCore
{
	enum class ETreeTraversalBehavior : uint8
	{
		/** Gives us the next element (may be child or neigbhour) */
		Continue,
		/** Gives us the next neighbour element - do not list any children of this node. */
		SkipSubtree,
		/** Stop iteration */
		Break
	};

	enum class EHierarchyObjectType
	{
		/**
		 * The object entry was added through an AddObject call.
		 * Example: AddObject(/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1)
		 */
		Explicit,
		/**
		 * The object entry was added indirectly because of a AddObject to a child.
		 * Example: AddObject(/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1) adds /Game/Maps.Map:PersistentLevel.Cube implicitly.
		 */
		Implicit
	};
	
	struct FObjectHierarchyInfo
	{
		FSoftObjectPath Object;
		EHierarchyObjectType Type;
	};
	
	struct FChildRelation
	{
		FObjectHierarchyInfo Parent;
		FObjectHierarchyInfo Child;
	};
	
	/** Keeps track of the outer tree hierarchy of FSoftObjectPath. */
	class FObjectPathHierarchy
		// This class cannot be shallowly copied because the nodes' referenced dynamic allocations - we don't expect instances to be copied, so we won't bother implementing a deep copy.
		: public FNoncopyable
	{
	public:

		FObjectPathHierarchy() = default;
		UE_API FObjectPathHierarchy(FObjectPathHierarchy&& Other);
		UE_API FObjectPathHierarchy& operator=(FObjectPathHierarchy&& Other);

		/**
		 * Traverses the hierarchy in pre-order (root first, then its children), starting at an optional parent object.
		 *
		 * Example sequence of callback invocations:
		 * 1. (/Game/Maps.Map,							/Game/Maps.Map:PersistentLevel)
		 * 2. (/Game/Maps.Map:PersistentLevel,			/Game/Maps.Map:PersistentLevel.Cube)
		 * 3. (/Game/Maps.Map:PersistentLevel.Cube,		/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0)
		 * 4. (/Game/Maps.Map:PersistentLevel.Cube,		/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1)
		 * 5. (/Game/Maps.Map:PersistentLevel,			/Game/Maps.Map:PersistentLevel.Sphere)
		 * 6. (/Game/Maps.Map:PersistentLevel.Sphere,	/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0)
		 *
		 * @important This lists implicit parents.
		 * For example, if you called AddObject(/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0), then its outers will be implicitly
		 * considered part of the hierarchy (i.e. /Game/Maps.Map:PersistentLevel.Cube, /Game/Maps.Map:PersistentLevel, /Game/Maps.Map).
		 * Use the FChildRelation to determine whether an object is part of the hiearchy implicitly or explicitly.
		 *
		 * Assets without subobjects are not listed.
		 * For example, if you called AddObject(/Game/Maps.Map) and the hierarchy was otherwise empty, Callback would not be invoked.
		 * 
		 * @param Callback The callback to invoke 
		 * @param Start The object at which to start walking down the hierarchy
		 */
		UE_API void TraverseTopToBottom(TFunctionRef<ETreeTraversalBehavior(const FChildRelation& Relation)> Callback, const FSoftObjectPath& Start = {}) const;

		/**
		 * Traverses the hierarchy in post-order (children first, then the root), starting at an optional parent object.
		 *
		 * Example sequence of callback invocations:
		 * 1. (/Game/Maps.Map:PersistentLevel.Cube,		/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0)
		 * 2. (/Game/Maps.Map:PersistentLevel.Cube,		/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1)
		 * 3. (/Game/Maps.Map:PersistentLevel,			/Game/Maps.Map:PersistentLevel.Cube)
		 * 4. (/Game/Maps.Map:PersistentLevel.Sphere,	/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0)
		 * 5. (/Game/Maps.Map:PersistentLevel,			/Game/Maps.Map:PersistentLevel.Sphere)
		 * 6. (/Game/Maps.Map,							/Game/Maps.Map:PersistentLevel)
		 *
		 * @important This lists implicit parents.
		 * For example, if you called AddObject(/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0), then its outers will be implicitly
		 * considered part of the hierarchy (i.e. /Game/Maps.Map:PersistentLevel.Cube, /Game/Maps.Map:PersistentLevel, /Game/Maps.Map).
		 * Use the FChildRelation to determine whether an object is part of the hiearchy implicitly or explicitly.
		 *
		 * Assets without children are not listed.
		 * For example, if you called AddObject(/Game/Maps.Map) and the hierarchy was otherwise empty, Callback would not be invoked.
		 * 
		 * @param Callback The callback to invoke 
		 * @param Start The object at which to start walking up the hierarchy
		 */
		UE_API void TraverseBottomToTop(TFunctionRef<EBreakBehavior(const FChildRelation& Relation)> Callback, const FSoftObjectPath& Start = {}) const;

		/** @return A valid TOptional if the object is in the hierarchy with the value indicating whether implicitly or explicitly. Unset if the object does not appear in the hierarchy.*/
		UE_API TOptional<EHierarchyObjectType> IsInHierarchy(const FSoftObjectPath& Object) const;
		/** @return Whether Object has any subobjects in the hierarchy. */
		UE_API bool HasChildren(const FSoftObjectPath& Object) const;
		/** @return Whether Object is an asset, i.e. a top-level asset that has never have any parents. */
		UE_API bool IsAssetInHierarchy(const FSoftObjectPath& Object) const;

		/** Checks there is an hierarchy. */
		bool IsEmpty() const { return AssetNodes.IsEmpty(); }
		
		/**
		 * Adds Object to the hierarchy. 
		 * Henceforth, the object shall be tracked as EHierarchyObjectType::Explicit.
		 * Calling this more than once has no effect.
		 */
		UE_API void AddObject(const FSoftObjectPath& ObjectPath);

		/**
		 * Removes Object from the hierarchy.
		 *
		 * If the object has no children, the object is removed entirely.
		 * If the object has children, it shall henceforth be tacked as EHierarchyObjectType::Implicit.
		 */
		UE_API void RemoveObject(const FSoftObjectPath& Object);

		/** Empties the entire hierarchy to the empty state. */
		void Clear() { AssetNodes.Reset(); CachedNodes.Reset(); }

	private:

		struct FTreeNode
		{
			FObjectHierarchyInfo Data;
			FTreeNode* Parent = nullptr;
			TArray<TUniquePtr<FTreeNode>> Children;

			explicit FTreeNode(FObjectHierarchyInfo Data)
				: Data(MoveTemp(Data))
			{}
			bool IsAssetNode() const { return Parent == nullptr; }
		};
		
		/**
		 * Each node represents an asset, e.g. /Game/Maps.Map.
		 * 
		 * That means every node in this array is of the form /package/path.assetname (i.e. FSoftObjectPath::AssetPath is set and FSoftObjectPath::SubPathString is empty).
		 * Remember in general FSoftObjectPath strings look like this: /package/path.assetname[:subpath] where /package/path.assetname is the FSoftObjectPath::AssetPath.
		 */
		TArray<TUniquePtr<FTreeNode>> AssetNodes;
		/** Maps every path to the corresponding node. */
		TMap<FSoftObjectPath, FTreeNode*> CachedNodes;

		static UE_API ETreeTraversalBehavior TraverseTopToBottomInternal(
			const FObjectHierarchyInfo& Owner,
			const TArray<TUniquePtr<FTreeNode>>& Nodes,
			TFunctionRef<ETreeTraversalBehavior(const FChildRelation& Relation)> Callback
			);
		static UE_API EBreakBehavior TraverseBottomToTopInternal(
			const FObjectHierarchyInfo& Owner,
			const TArray<TUniquePtr<FTreeNode>>& Nodes,
			TFunctionRef<EBreakBehavior(const FChildRelation& Relation)> Callback
			);
	};
}

#undef UE_API
