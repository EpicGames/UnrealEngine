// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectPathUtils.h"

#include "Misc/Optional.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertSyncCore
{
	/**
	 * Parses a FSoftObjectPath string and iterates through all outer objects of the Start path.
	 * 
	 * Example: Start = /Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0.Foo would invoke Callback in this order:
	 * 1. /Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0
	 * 2. /Game/Maps.Map:PersistentLevel.Cube
	 * 3. /Game/Maps.Map:PersistentLevel
	 * 4. /Game/Maps.Map
	 *
	 * Example usage:
	 * const FSoftObjectPath ComponentPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0") };
	 * for (ConcertSyncCore::FObjectPathOuterIterator It(ComponentPath); It; ++It)
	 * {
	 *		const FSoftObjectPath& Path = *It; // 1st iteration = /Game/Maps.Map:PersistentLevel.Cube
	 *		const FString String = It->ToString(); // 1st iteration = /Game/Maps.Map:PersistentLevel.Cube
	 * }
	 */
	class FObjectPathOuterIterator
	{
	public:

		explicit FObjectPathOuterIterator(const FSoftObjectPath& Start)
			: Current(Start)
		{
			Advance();
		}

		FORCEINLINE void operator++() { Advance(); }
		
		FORCEINLINE explicit operator bool() const {  return Current.IsSet();  }
		FORCEINLINE bool operator !() const  { return !static_cast<bool>(*this); }
		
		FORCEINLINE const FSoftObjectPath& operator*() const { check(Current); return *Current; }
		FORCEINLINE const FSoftObjectPath* operator->() const { return Current.GetPtrOrNull(); }
		
		FORCEINLINE bool operator==(const FObjectPathOuterIterator& Right) const { return Current == Right.Current; }
		FORCEINLINE bool operator!=(const FObjectPathOuterIterator& Right) const { return Current != Right.Current; }
		
	private:

		TOptional<FSoftObjectPath> Current;

		void Advance()
		{
			if (Current)
			{
				Current = GetOuterPath(*Current);
			}
		}
	};
}