// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Utf8String.h"
#include "Misc/Optional.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertSyncCore
{
	/**
	 * Gets the outer of ObjectPath.
	 *
	 * Examples:
	 * - GetOuterPath(/Game/Map.Map:PersistentLevel.Actor.Component)	= /Game/Map.Map:PersistentLevel.Actor
	 * - GetOuterPath(/Game/Map.Map:PersistentLevel.Actor)				= /Game/Map.Map:PersistentLevel
	 * - GetOuterPath(/Game/Map.Map:PersistentLevel)					= /Game/Map.Map
	 * - GetOuterPath(/Game/Map.Map)									= unset optional
	 * - GetOuterPath({})												= unset optional
	 * 
	 * @return The outer of ObjectPath, if there is one. Unset if ObjectPath points to a package.
	 */
	TOptional<FSoftObjectPath> GetOuterPath(const FSoftObjectPath& ObjectPath);
}

namespace UE::ConcertSyncCore
{
	inline TOptional<FSoftObjectPath> GetOuterPath(const FSoftObjectPath& ObjectPath)
	{
		if (ObjectPath.IsNull() || !ObjectPath.IsSubobject())
		{
			return {};
		}
		
		FUtf8String SubPathString = ObjectPath.GetSubPathUtf8String();
		const int32 CurrentIndex = SubPathString.Find(".", ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (SubPathString.IsValidIndex(CurrentIndex))
		{
			SubPathString.LeftInline(CurrentIndex);
			return FSoftObjectPath{ ObjectPath.GetAssetPath(), *SubPathString };
		}
		else
		{
			return FSoftObjectPath{ ObjectPath.GetAssetPath(), {} };
		}
	}
}
