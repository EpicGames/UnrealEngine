// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertSyncCore
{
	/** @return Checks whether the given object is an actor. */
	CONCERTSYNCCORE_API bool IsActor(const FSoftObjectPath& Object);
	
	/** @return Get the owning actor of Subobject. If Subobject is an actor, then this returns unset. */
	CONCERTSYNCCORE_API TOptional<FSoftObjectPath> GetActorOf(const FSoftObjectPath& Subobject);
	/** @return The actor portion of the path. Difference to GetActorOf: returns a set value whenever the path is an object in the level (actor, component, etc.) */
	FORCEINLINE TOptional<FSoftObjectPath> GetActorPathIn(const FSoftObjectPath& Path) { return IsActor(Path) ? TOptional(Path) : GetActorOf(Path); }
	
	/** @return Gets the last object name in the subpath. */
	CONCERTSYNCCORE_API FUtf8String ExtractObjectNameFromPath(const FSoftObjectPath& Object);

	/**
	 * Replaces the package path and actor name from NewActor witht that of NewActor.
	 * 
	 * Valid example 1:
	 * - OldPath:		/Game/OldMap.OldMap:PersistentLevel.OldActor.Subobject
	 * - NewActor:		/Game/NewMap.NewMap:PersistentLevel.NewActor
	 * - Result:		/Game/NewMap.NewMap:PersistentLevel.NewActor.Subobject
	 *
	 * Invalid examples:
	 * - Missing actor path:
	 *	- OldPath:		/Game/Map.Map.Actor,	NewPath: /Game/Map.Map
	 *	- OldPath:		/Game/Map.Map,			NewPath: /Game/Map.Map.Actor
	 * - Subobject in NewActor are not allowed:
	 *	- OldPath:		/Game/Map.Map:PersistentLevel.OldActor.Subobject
	 *	- NewActor:		/Game/Map.Map:PersistentLevel.NewActor.Subobject
	 *
	 * @param OldPath A path to an object in a UWorld in which the actor is supposed to be replaced
	 * @param NewActor A path to an actor
	 *
	 * @return The replaced object paths. Unset if the arguments were invalid.
	 */
	CONCERTSYNCCORE_API TOptional<FSoftObjectPath> ReplaceActorInPath(const FSoftObjectPath& OldPath, const FSoftObjectPath& NewActor);
};
