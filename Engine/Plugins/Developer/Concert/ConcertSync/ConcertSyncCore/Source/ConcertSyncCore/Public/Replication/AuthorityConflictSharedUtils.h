// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Misc/EBreakBehavior.h"
#include "Templates/Function.h"

struct FConcertPropertyChain;
struct FConcertPropertySelection;
struct FConcertReplication_ChangeAuthority_Request;
struct FConcertReplication_ChangeStream_Request;
struct FConcertObjectReplicationMap;
struct FConcertReplicatedObjectId;

namespace UE::ConcertSyncCore::Replication { class IReplicationGroundTruth; }

namespace UE::ConcertSyncCore::Replication::AuthorityConflictUtils
{
	enum class EAuthorityConflict : uint8
	{
		/** The client is allowed to take authority */
		Allowed,
		/** There was at least one conflict */
		Conflict,
	};
	
	using FProcessAuthorityConflict = TFunctionRef<EBreakBehavior(const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertyChain& ConflictingProperty)>;
	
	/**
	 * Enumerates all authority conflicts, if any, that would occur if Object were to be replicated with the given Properties.
	 *
	 * @param ClientId The client that wants to take authority 
	 * @param Object Describes the object, the client, and the stream to be replicated.
	 * @param OverwriteProperties The properties that will be registered with the object (i.e. not what IClientInfoSource reports currently)
	 * @param GroundTruth Provides information about clients. On the server, it is the definite state whereas on client machines it would be
	 * what the local client thinks what the server state is.
	 * @param ProcessConflict Callback to be called for each permutation of conflicting client, stream and property.
	 * 
	 * @return Whether there were any conflicts
	 */
	CONCERTSYNCCORE_API EAuthorityConflict EnumerateAuthorityConflicts(
		const FGuid& ClientId,
		const FSoftObjectPath& Object,
		const TSet<FConcertPropertyChain>& OverwriteProperties,
		const IReplicationGroundTruth& GroundTruth,
		FProcessAuthorityConflict ProcessConflict = [](const FGuid&, const FGuid&, const FConcertPropertyChain&){ return EBreakBehavior::Break; }
		);

	/**
	 * Removes entries from Request that would generate conflicts.
	 * 
	 * @param Request The request to clense
	 * @param SendingClient The client that will send the request
	 * @param GroundTruth Provides information about clients. On the server, it is the definite state whereas on client machines it would be
	 * what the local client thinks what the server state is.
	 */
	CONCERTSYNCCORE_API void CleanseConflictsFromAuthorityRequest(FConcertReplication_ChangeAuthority_Request& Request, const FGuid& SendingClient, const IReplicationGroundTruth& GroundTruth);
	
	/**
	 * Removes entries from Request that would generate conflicts.
	 * 
	 * @param Request The request to clense
	 * @param SendingClient The client that will send the request
	 * @param GroundTruth Provides information about clients. On the server, it is the definite state whereas on client machines it would be
	 * what the local client thinks what the server state is.
	 */
	CONCERTSYNCCORE_API void CleanseConflictsFromStreamRequest(FConcertReplication_ChangeStream_Request& Request, const FGuid& SendingClient, const IReplicationGroundTruth& GroundTruth);
}
