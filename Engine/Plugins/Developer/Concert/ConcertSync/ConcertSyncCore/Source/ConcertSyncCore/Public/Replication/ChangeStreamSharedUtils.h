// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Messages/ChangeStream.h"
#include "Templates/Function.h"

enum class EBreakBehavior : uint8;

struct FConcertReplication_ChangeStream_Request;
struct FConcertObjectInStreamID;
struct FConcertObjectReplicationMap;
struct FConcertReplicationStream;
struct FConcertReplicatedObjectInfo;

/** This namespace contains the shared logic for applying FConcertChangeStream_Requests. */
namespace UE::ConcertSyncCore::Replication::ChangeStreamUtils
{
	/**
	 * Iterates through all objects that are removed and therefore the requesting client would lose authority over.
	 * This function assumes that Request was validated.
	 * 
	 * @param Request The request to parse
	 * @param ExistingStreams The streams the request would be applied to
	 * @param Callback The callback to process the objects
	 */
	CONCERTSYNCCORE_API void ForEachRemovedObject(
		const FConcertReplication_ChangeStream_Request& Request,
		const TArray<FConcertReplicationStream>& ExistingStreams,
		TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID&)> Callback
		);

	/**
	 * Iterates through all objects that are added by Request but not yet in ExistingStreams.
	 * This function assumes that Request was validated.
	 *
	 * @param Request The request to parse
	 * @param ExistingStreams The streams the request would be applied to
	 * @param Callback The callback to process the objects
	 */
	CONCERTSYNCCORE_API void ForEachAddedObject(
		const FConcertReplication_ChangeStream_Request& Request,
		const TArray<FConcertReplicationStream>& ExistingStreams,
		TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID&)> Callback
		);
	
	/**
	 * Modifies RegisteredStreams as described by the request without validating that the Request is valid to apply.
	 * @param Request The request to parse
	 * @param StreamsToModify The streams to apply the request to
	 */
	CONCERTSYNCCORE_API void ApplyValidatedRequest(
		const FConcertReplication_ChangeStream_Request& Request,
		IN OUT TArray<FConcertReplicationStream>& StreamsToModify
		);
	
	/**
	 * Modifies StreamToModify as described by the request.
	 * 
	 * Validate the request before calling.
	 * This function does use ensure conditions where it makes sense but it does not check all failure cases.
	 * 
	 * @param Request The request to parse
	 * @param SettingsToModify The streams to apply the request to
	 */
	CONCERTSYNCCORE_API void ApplyValidatedFrequencyChanges(
		const FConcertReplication_ChangeStream_Frequency& Request,
		FConcertStreamFrequencySettings& SettingsToModify
		);
	/**
	 * Validates a FConcertReplication_ChangeStream_Request and optionally outputs the errors into OptionalErrors if set.
	 *
	 * @param Request The request to validate. Note that the full request is needed (if an object is added through this
	 * @param Streams The streams array Request would be run on
	 * @param OptionalErrors Errors are written into this struct if set
	 *
	 * @return Whether Request is valid to apply to Streams
	 */
	CONCERTSYNCCORE_API bool ValidateFrequencyChanges(
		const FConcertReplication_ChangeStream_Request& Request,
		const TConstArrayView<FConcertReplicationStream>& Streams,
		OUT FConcertReplication_ChangeStream_FrequencyResponse* OptionalErrors = nullptr
		);

	/** Iterates entries with incomplete data, which are entries for which IsValidForSendingToServer returns false. */
	CONCERTSYNCCORE_API void IterateInvalidEntries(const FConcertObjectReplicationMap& ReplicationMap, TFunctionRef<EBreakBehavior(const FSoftObjectPath&, const FConcertReplicatedObjectInfo&)> Callback);

	/**
	 * Builds PutObject and RemoveObject requests that would transform Base to Desired.
	 * 
	 * The state of an object in a replication map is only valid if all its members have values:
	 * that means the class is non-null and not the property selection is non-empty.
	 * This function enforces (and ensure()s) that the Base and Desired states are valid.
	 * The changes are validated using FConcertChangeStream_PutObject's factories MakeFromInfo and MakeFromChange.
	 * 
	 * @param StreamId The ID of the changed stream: it is inserted into the request
	 * @param Base The base stream
	 * @param Desired The state the stream should end up in 
	 * @return A semantically correct FConcertChangeStream_Request which Base to Desired (note is can be rejected due to authority conflicts).
	 */
	CONCERTSYNCCORE_API FConcertReplication_ChangeStream_Request BuildRequestFromDiff(
		const FGuid& StreamId,
		const FConcertObjectReplicationMap& Base,
		const FConcertObjectReplicationMap& Desired
		);
}
