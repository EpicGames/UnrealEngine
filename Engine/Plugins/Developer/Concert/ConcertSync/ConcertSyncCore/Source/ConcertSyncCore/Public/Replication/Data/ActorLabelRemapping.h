// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

#include "ActorLabelRemapping.generated.h"

/** Metadata about an actor in a FConcertObjectReplicationMap. */
USTRUCT()
struct FConcertReplicationRemappingData_Actor
{
	GENERATED_BODY()
	
	/**
	 * Label of the actor.
	 * 
	 * This is typically AActor::ActorLabel.
	 * Practically, the API is generic enough to allow using some other actor identifying string at runtime instead if you really wanted to.
	 */
	UPROPERTY()
	FString Label;

	/**
	 * The class the actor had when the remapping data was generated.
	 * 
	 * Note that the actor's itself may not even have been in the FConcertObjectReplicationMap when the data was generated.
	 * During generation, when processing a component (or other subobject), the owning actor instance can be looked up by path and its class is recorded here.
	 * If the actor does not resolve, it is not recorded.
	 *
	 * Example:
	 *  - Suppose the replication contains a single component: "/Game/World.World:PersistentLevel.Actor.Component"
	 *	- If during data generation, the class of "/Game/World.World:PersistentLevel.Actor" 
	 *		- can be obtained, the corresponding FConcertReplicationRemappingData::ActorData contains the actor path and its class.
	 *		- can not be obtained, then FConcertReplicationRemappingData::Actor.IsEmpty()
	 */
	UPROPERTY()
	FSoftClassPath Class;
};

/**
 * Supplementary data for a FConcertObjectReplicationMap.
 * This data can be used to remap FSoftObjectPaths in FConcertObjectReplicationMap to different FConcertObjectReplicationMap based on AActor::ActorLabel.
 * The primary use case is replication presets being translated multiple sessions.
 * 
 * Example:
 * 1. In session 1, suppose a FConcertObjectReplicationMap is created like this:
 *		- { "/Game/World.World:PersistentLevel.ActorOld", { "ActorTags" } }
 *		- { "/Game/World.World:PersistentLevel.ActorOld.StaticMeshComponentOld", { "RelativeLocation", "RelativeLocation.X" } }
 *		- Actor_0 has the actor label "MyStaticMeshActor"
 * 2. As part of creating the FConcertObjectReplicationMap, you also create a FConcertReplicationRemappingData (@see ActorLabelRemappingCore.h)
 * 3. In session 2, suppose there is world:
 *		- /Game/OtherWorld.OtherWorld:PersistentLevel.ActorNew
 *		- /Game/OtherWorld.OtherWorld.PersistentLevel.ActorNew.StaticMeshComponentNew
 *		- OtherActor_0 has the actor label "MyStaticMeshActor"
 * In the above scenario, you can now remap the FConcertObjectReplicationMap from step 1 using the FConcertReplicationRemappingData from step 2.
 * Effectively, "ActorNew" would get the properties of "ActorOld", and "StaticMeshComponentNew" the properties of "StaticMeshComponentOld", because
 * the labels of "ActorNew" and "ActorOld" are both "MyStaticMeshActor".
 *
 * @see ActorLabelRemappingUtils.h
 */
USTRUCT()
struct FConcertReplicationRemappingData
{
	GENERATED_BODY()
	
	/** Data about actors. */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertReplicationRemappingData_Actor> ActorData;
};
