// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "ObjectIds.h"
#include "AuthorityConflict.generated.h"

/** Describes an authority conflict for two objects. */
USTRUCT()
struct FConcertAuthorityConflict
{
	GENERATED_BODY()

	/** The object that was attempted to take authority over but could not be. */
	UPROPERTY()
	FConcertObjectInStreamID AttemptedObject;

	/** The object of another client that Source conflicts with. */
	UPROPERTY()
	FConcertReplicatedObjectId ConflictingObject;
};

USTRUCT()
struct FConcertAuthorityConflictArray
{
	GENERATED_BODY()

	/** Each AttemptedObject is unique. */
	UPROPERTY()
	TArray<FConcertAuthorityConflict> Conflicts;

	FConcertAuthorityConflict& FindOrAdd(const FConcertObjectInStreamID& AttemptedObject)
	{
		FConcertAuthorityConflict* Result = Conflicts.FindByPredicate([&AttemptedObject](const FConcertAuthorityConflict& Conflict)
		{
			return Conflict.AttemptedObject == AttemptedObject;
		});
		if (Result)
		{
			return *Result;
		}

		const int32 Index = Conflicts.Add({ AttemptedObject });
		return Conflicts[Index];
	}
};