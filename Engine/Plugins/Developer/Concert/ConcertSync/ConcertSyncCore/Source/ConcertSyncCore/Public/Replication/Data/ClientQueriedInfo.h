// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectIds.h"
#include "ReplicationStream.h"
#include "ClientQueriedInfo.generated.h"

/** Describes objects a client has authority over */
USTRUCT()
struct FConcertAuthorityClientInfo
{
	GENERATED_BODY()

	/** The stream ID the client has registered */
	UPROPERTY()
	FGuid StreamId;

	/**
	 * The objects the client has authority over.
	 * You can figure out the properties by inspecting the corresponding client's stream using the object path.
	 */
	UPROPERTY()
	TArray<FSoftObjectPath> AuthoredObjects;
};

/** This is info a client receives about another client via FConcertQueryClientStreams_Request. */
USTRUCT()
struct FConcertQueriedClientInfo
{
	GENERATED_BODY()

	/**
	 * The streams the client has registered.
	 * 
	 * If EConcertQueryClientStreamFlags::SkipStreamInfo was set, this is empty.
	 * If EConcertQueryClientStreamFlags::SkipProperties was set, streams' FConcertReplicatedObjectInfo::PropertySelection are empty.
	 * If EConcertQueryClientStreamFlags::SkipFrequency was set, the FrequencySettings are empty.
	 */
	UPROPERTY()
	TArray<FConcertBaseStreamInfo> Streams;

	/**
	 * Indirectly describes which object properties the client has authority over.
	 *
	 * If EConcertQueryClientStreamFlags::SkipAuthority was set, this is empty.
	 */
	UPROPERTY()
	TArray<FConcertAuthorityClientInfo> Authority;

	bool IsEmpty() const { return Streams.IsEmpty() && Authority.IsEmpty(); }
	
	/** @return Whether Object is owned */
	bool HasAuthority(const FConcertObjectInStreamID& Object) const
	{
		const FConcertAuthorityClientInfo* Info = Authority.FindByPredicate([StreamId = Object.StreamId](const FConcertAuthorityClientInfo& Info){ return Info.StreamId == StreamId; });
		return Info && Info->AuthoredObjects.Contains(Object.Object);
	}
};