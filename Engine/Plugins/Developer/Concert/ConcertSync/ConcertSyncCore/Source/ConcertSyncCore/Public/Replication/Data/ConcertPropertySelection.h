// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"
#include "ConcertPropertySelection.generated.h"

#define UE_API CONCERTSYNCCORE_API

class UStruct;
struct FArchiveSerializedPropertyChain;

/**
 * Describes the path to a FProperty replicated by Concert.
 * @see FConcertPropertyChain::PathToProperty.
 */
USTRUCT()
struct FConcertPropertyChain
{
	GENERATED_BODY()

	/** Constructs a FConcertPropertyChain from a path if it is valid. If you need to create many paths in one go, use PropertyUtils::BulkConstructConcertChainsFromPaths instead. */
	static UE_API TOptional<FConcertPropertyChain> CreateFromPath(const UStruct& Class, const TArray<FName>& NamePath);

	FConcertPropertyChain() = default;
	/**
	 * @param OptionalChain The chain leading up to LeafProperty. If it is a root property it can either be empty or nullptr. This mimics the behaviours of FArchive.
	 * @param LeafProperty The property the chain leads to. It will be the last property in PathToProperty. This can be the inner property of a container but ONLY for primitive types (float, etc.) or structs with a custom Serialize function. 
	 */
	UE_API FConcertPropertyChain(const FArchiveSerializedPropertyChain* OptionalChain, const FProperty& LeafProperty);
	
	/** Gets the leaf property, which is the property the path leads towards. */
	FName GetLeafProperty() const { return IsEmpty() ? NAME_None : PathToProperty[PathToProperty.Num() - 1]; }
	FName GetRootProperty() const { return IsEmpty() ? NAME_None : PathToProperty[0]; }
	bool IsRootProperty() const { return PathToProperty.Num() == 1; }
	bool IsEmpty() const { return PathToProperty.IsEmpty(); }

	/** @return Whether this is a parent of ChildToCheck */
	bool IsParentOf(const FConcertPropertyChain& ChildToCheck) const { return ChildToCheck.IsChildOf(*this); }
	
	/** @return Whether the leaf property is a child of the given property chain. */
	UE_API bool IsChildOf(const FConcertPropertyChain& ParentToCheck) const;
	/** @return Whether the leaf property is a direct child of the given property chain. */
	UE_API bool IsDirectChildOf(const FConcertPropertyChain& ParentToCheck) const;

	/**
	 * Utility for checking whether this path corresponds to OptionalChain leading to LeafProperty.
	 * 
	 * @param OptionalChain Contains all properties leading up to the LeafProperty. You can skip inner container properties.
	 * @param LeafProperty The last property
	 * @return Whether OptionalChain and LeafProperty correspond to this path. */
	UE_API bool MatchesExactly(const FArchiveSerializedPropertyChain* OptionalChain, const FProperty& LeafProperty) const;

	/** @return Attempts to resolve this property given the class */
	UE_API FProperty* ResolveProperty(const UStruct& Class, bool bLogOnFail = true) const;

	/** @return The property immediately before the current one in the chain */
	UE_API FConcertPropertyChain GetParent() const;
	/** @return The root-most parent in the chain. */
	UE_API FConcertPropertyChain GetRootParent() const;
	/** @return The property path */
	const TArray<FName>& GetPathToProperty() const { return PathToProperty; }

	enum class EToStringMethod
	{
		Path,
		LeafProperty
	};
	UE_API FString ToString(EToStringMethod Method = EToStringMethod::Path) const;
	
	friend bool operator==(const FConcertPropertyChain& Left, const FConcertPropertyChain& Right) { return Left.PathToProperty == Right.PathToProperty; }
	friend bool operator!=(const FConcertPropertyChain& Left, const FConcertPropertyChain& Right) { return !(Left == Right); }

	friend bool operator==(const FConcertPropertyChain& Left, const TArray<FName>& Path) { return Left.PathToProperty == Path; }
	friend bool operator!=(const FConcertPropertyChain& Left, const TArray<FName>& Path) { return Left.PathToProperty != Path; }
	
	friend bool operator==(const TArray<FName>& Path, const FConcertPropertyChain& Left) { return Left.PathToProperty == Path; }
	friend bool operator!=(const TArray<FName>& Path, const FConcertPropertyChain& Left) { return Left.PathToProperty != Path; }
	
private:
	
	/**
	 * Path from root of UObject to leaf property. Includes the leaf property.
	 * Inner container properties, i.e. FArrayProperty::Inner, FSetProperty::ElementProp, FMapProperty::KeyProp, and FMapProperty::ValueProp, are
	 * never listed in the property path.
	 *
	 * Listing some paths (see example code below):
	 *  - { "Struct", "Foo" }
	 *  - { "ArrayOfStructs", "Foo" }
	 *  - { "MapOfStructs" }
	 *  - { "MapOfStructs", "Foo" } 
	 * See Concert.Replication.Data.ForEachReplicatableConcertProperty for path examples (just CTRL+SHIFT+F or go to ConcertSyncTest/Replication/ConcertPropertyTests.cpp).
	 *
	 * FConcertPropertyChains do NOT cross the UObject border. In the above example, there would be no such thing as { "UnsupportedInstanced", ... }.
	 * You'd start a new FConcertPropertyChain for properties for objects of type UFooSubobject.
	 * 
	 * This property is kept private to force the use of the exposed constructors.
	 *
	 * Code for example:
	 * class AFooActor
	 * {
	 *		UPROPERTY()
	 *		FFooStruct Struct;
	 * 
	 *		UPROPERTY()
	 *		TArray<FFooStruct> ArrayOfStructs;
	 *
	 *		UPROPERTY()
	 *		TMap<int32, FFooStruct> MapOfStructs;
	 *
	 *		UPROPERTY()
	 *		TMap<int32, float> MapIntToFloat;
	 *
	 *		UPROPERTY(Instanced)
	 *		TArray<UFooSubobject*> UnsupportedInstanced;
	 * };
	 *
	 * struct FFooStruct
	 * {
	 *		UPROPERTY()
	 *		int32 Foo;
	 *
	 *		UPROPERTY()
	 *		int32 Bar;
	 * };
	 */
	UPROPERTY()
	TArray<FName> PathToProperty;
};

/** List of properties to be replicated for a given object */
USTRUCT()
struct FConcertPropertySelection
{
	GENERATED_BODY()

	/** List of replicated properties. */
	UPROPERTY()
	TSet<FConcertPropertyChain> ReplicatedProperties;

	/** @return Whether this and Other contain at least one property that is the same. */
	bool OverlapsWith(const FConcertPropertySelection& Other) const { return EnumeratePropertyOverlaps(ReplicatedProperties, Other.ReplicatedProperties); }

	/** @return Whether this includes all properties of Other */
	bool Includes(const FConcertPropertySelection& Other) const { return ReplicatedProperties.Includes(Other.ReplicatedProperties); }

	/**
	 * Adds all parent properties if they are missing.
	 * 
	 * Suppose ReplicatedProperties = { ["Vector", "X"] }.
	 * After execution, it would be { ["Vector", "X"], ["Vector"] }
	 */
	UE_API void DiscoverAndAddImplicitParentProperties();
	
	/**
	 * Determines all properties that overlap.
	 * This algorithm is strictly O(n^2) but runs O(n) on average.
	 * 
	 * @return Whether there were any property overlaps.
	 */
	static UE_API bool EnumeratePropertyOverlaps(
		const TSet<FConcertPropertyChain>& First,
		const TSet<FConcertPropertyChain>& Second,
		TFunctionRef<EBreakBehavior(const FConcertPropertyChain&)> Callback = [](const FConcertPropertyChain&){ return EBreakBehavior::Break; }
		);
	
	friend bool operator==(const FConcertPropertySelection& Left, const FConcertPropertySelection& Right)
	{
		return Left.ReplicatedProperties.Num() == Right.ReplicatedProperties.Num() && Left.Includes(Right);
	}
	friend bool operator!=(const FConcertPropertySelection& Left, const FConcertPropertySelection& Right)
	{
		return !(Left == Right);
	}
};

namespace UE::ConcertSyncCore
{
	/**
	 * Implementation of hashing FConcertPropertyChain. Allows you to use TSet::ContainsByHash without constructing a FConcertPropertyChain, which is expensive because it searches the property tree.
	 * You can rely on the fact that this function is either updated or deprecated when the hasing algorithm for FConcertPropertyChain is changed.
	 */
	CONCERTSYNCCORE_API uint32 ComputeHashForPropertyChainContent(const TArray<FName>& PropertyChain);
}
inline uint32 GetTypeHash(const FConcertPropertyChain& Chain)
{
	// If you need to changing the hashing function - update ComputeHashForPropertyChainContent since some code relies on the hasing logic.
	return UE::ConcertSyncCore::ComputeHashForPropertyChainContent(Chain.GetPathToProperty());
}

#undef UE_API
