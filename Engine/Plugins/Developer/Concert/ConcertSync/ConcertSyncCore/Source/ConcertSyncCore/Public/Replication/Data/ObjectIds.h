// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "UObject/SoftObjectPath.h"
#include "ObjectIds.generated.h"

/** Utility for identifying an object part of a replication stream definition.*/
USTRUCT()
struct FConcertObjectInStreamID
{
	GENERATED_BODY()
	
	/** The replication stream that produces data for this object. */
	UPROPERTY()
	FGuid StreamId;
	/** The object to process */
	UPROPERTY()
	FSoftObjectPath Object;

	FString ToString() const { return FString::Printf(TEXT("StreamId: %s, Object: %s"), *StreamId.ToString(), *Object.ToString()); }
		
	friend bool operator==(const FConcertObjectInStreamID& Left, const FConcertObjectInStreamID& Right)
	{
		return Left.StreamId == Right.StreamId && Left.Object == Right.Object;
	}

	friend bool operator!=(const FConcertObjectInStreamID& Left, const FConcertObjectInStreamID& Right)
	{
		return !(Left == Right);
	}
};

/** Identifies an object that was replicated by a client based on an underlying stream. */
USTRUCT()
struct FConcertReplicatedObjectId : public FConcertObjectInStreamID
{
	GENERATED_BODY()
	
	/** The ID of the endpoint that sent this object. This can be the client or server endpoint depending on who receives it. */
	FGuid SenderEndpointId;
	
	FString ToString() const { return FString::Printf(TEXT("StreamId: %s, Object: %s, Sender: %s"), *StreamId.ToString(), *Object.ToString(), *SenderEndpointId.ToString()); }
	FString ToString(const FString& ClientName) const { return FString::Printf(TEXT("StreamId: %s, Object: %s, Sender: %s"), *StreamId.ToString(), *Object.ToString(), *ClientName); }
		
	friend bool operator==(const FConcertReplicatedObjectId& Left, const FConcertReplicatedObjectId& Right)
	{
		return Left.SenderEndpointId == Right.SenderEndpointId
			&& static_cast<const FConcertObjectInStreamID&>(Left) == static_cast<const FConcertObjectInStreamID&>(Right);
	}

	friend bool operator!=(const FConcertReplicatedObjectId& Left, const FConcertReplicatedObjectId& Right)
	{
		return !(Left == Right);
	}
};

USTRUCT()
struct FConcertObjectInStreamArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FConcertObjectInStreamID> Objects;
};

CONCERTSYNCCORE_API uint32 GetTypeHash(const FConcertObjectInStreamID& StreamObject);
CONCERTSYNCCORE_API uint32 GetTypeHash(const FConcertReplicatedObjectId& StreamObject);
