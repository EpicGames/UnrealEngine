// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"
#include "UObject/Object.h"
#include "ObjectReplicationMap.generated.h"

USTRUCT()
struct FConcertReplicatedObjectInfo
{
	GENERATED_BODY()

	/** The exact class of the object being replicated. */
	UPROPERTY()
	FSoftClassPath ClassPath;

	/** The properties to replicate for the object */
	UPROPERTY()
	FConcertPropertySelection PropertySelection;

	// Use static factories instead of user-provided constructors to continue allowing brace-initialization
	static FConcertReplicatedObjectInfo Make(const UObject& Object)
	{
		return { Object.GetClass() };
	}

	/** @return Whether this data is valid for sending to the server. */
	bool IsValidForSendingToServer() const
	{
		return ClassPath.IsValid()
			&& !PropertySelection.ReplicatedProperties.IsEmpty();
	}
	
	friend bool operator==(const FConcertReplicatedObjectInfo& Left, const FConcertReplicatedObjectInfo& Right)
	{
		return Left.ClassPath == Right.ClassPath
			&& Left.PropertySelection == Right.PropertySelection;
	}
	friend bool operator!=(const FConcertReplicatedObjectInfo& Left, const FConcertReplicatedObjectInfo& Right)
	{
		return !(Left == Right);
	}
};

/** Maps objects to their replicated properties. */
USTRUCT()
struct FConcertObjectReplicationMap
{
	GENERATED_BODY()

	/**
	 * List of replicated objects.
	 *
	 * Can be actors or its subobjects.
	 * Technically, this can also be non-UWorld objects.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertReplicatedObjectInfo> ReplicatedObjects;

	/** @return Whether ObjectPath has any properties assigned to it. */
	bool HasProperties(const FSoftObjectPath& ObjectPath) const
	{
		const FConcertReplicatedObjectInfo* ObjectInfo = ReplicatedObjects.Find(ObjectPath);
		return ObjectInfo && !ObjectInfo->PropertySelection.ReplicatedProperties.IsEmpty();
	}

	/** @return Whether there are any properties assigned. */
	bool IsEmpty() const
	{
		return ReplicatedObjects.IsEmpty();
	}

	friend bool operator==(const FConcertObjectReplicationMap& Left, const FConcertObjectReplicationMap& Right)
	{
		return Left.ReplicatedObjects.OrderIndependentCompareEqual(Right.ReplicatedObjects);
	}
	friend bool operator!=(const FConcertObjectReplicationMap& Left, const FConcertObjectReplicationMap& Right)
	{
		return !(Left == Right);
	}
};