// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Processing/Actions/ConcertReplicationAction.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/FieldPath.h"
#include "ReplicationActionEntry.generated.h"

/** Defines an action that should be performed when a property is replicated. */
USTRUCT()
struct FConcertReplicationActionEntry
{
	GENERATED_BODY()

	/** The action to perform when a property is matched. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	TInstancedStruct<FConcertReplicationAction> Action;

	/** If any of these properties is replicated, perform Action on the replicated object. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	TArray<TFieldPath<FProperty>> Properties;
};