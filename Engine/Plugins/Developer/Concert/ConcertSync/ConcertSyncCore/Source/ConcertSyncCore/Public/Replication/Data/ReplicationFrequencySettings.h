// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformTime.h"
#include "ReplicationFrequencySettings.generated.h"

UENUM()
enum class EConcertObjectReplicationMode : uint8
{
	/** Replicate at the rate specified at FConcertReplicatedObjectInfo::ReplicationRate */
	SpecifiedRate,
	/** Replicate the object as often as possible: every tick. */
	Realtime,

	/***** ADD NEW ENTRIES ABOVE THIS LINE *****/
	
	/** Not a real mode. */
	Count
};

namespace UE::ConcertSyncCore
{
	enum class ECanProcessObject : uint8
	{
		/** Enough time has elapsed: the object can be processed. */
		CanProcess,
		/** Not enough time has elapsed: wait with processing the object. */
		TooEarly
	};
}

/** Specifies replication frequency settings for a single object. */
USTRUCT()
struct FConcertObjectReplicationSettings
{
	GENERATED_BODY()
	
	/** Controls how often this object should be replicated. */
	UPROPERTY(EditAnywhere, Category = "Concert")
	EConcertObjectReplicationMode ReplicationMode = EConcertObjectReplicationMode::SpecifiedRate;
	
	/**
	 * How often per second the object is supposed to be replicated per second.
	 * 
	 * The update in seconds is given by 1 / ReplicationRate.
	 * The default rate of 30 results in an update interval every 0.033s
	 */
	UPROPERTY(EditAnywhere, Category = "Concert")
	uint8 ReplicationRate = 30;

	bool IsValid() const { return ReplicationMode == EConcertObjectReplicationMode::Realtime || ReplicationRate > 0; }
	
	/**
	 * Whether an object with this config can be replicated given the current time and the time it was last replicated.
	 * @param LastTimeProcessed The time the object was last processed - past result of FPlatformTime::Seconds()
	 * @param Now The current time - result of FPlatformTime::Seconds(). Pass this in to avoid excessive time spent on timing
	 */
	UE::ConcertSyncCore::ECanProcessObject CanProcessObject(const double LastTimeProcessed, const double Now = FPlatformTime::Seconds()) const
	{
		switch (ReplicationMode)
		{
		case EConcertObjectReplicationMode::SpecifiedRate:
			return LastTimeProcessed + GetUpdateIntervalInSeconds() <= Now
				? UE::ConcertSyncCore::ECanProcessObject::CanProcess
				: UE::ConcertSyncCore::ECanProcessObject::TooEarly;
			
		case EConcertObjectReplicationMode::Realtime: return UE::ConcertSyncCore::ECanProcessObject::CanProcess;
		default:
			checkNoEntry();
			return UE::ConcertSyncCore::ECanProcessObject::CanProcess;
		}
	}
	
	/** Gets the time to wait between frames replicating an object with this config. */
	double GetUpdateIntervalInSeconds() const
	{
		return ReplicationMode == EConcertObjectReplicationMode::Realtime
			? 0.0
			: 1.0 / static_cast<double>(ReplicationRate);
	}

	friend bool operator==(const FConcertObjectReplicationSettings& Left, const FConcertObjectReplicationSettings& Right)
	{
		return Left.ReplicationMode == Right.ReplicationMode
			&& Left.ReplicationRate == Right.ReplicationRate;
	}
	friend bool operator!=(const FConcertObjectReplicationSettings& Left, const FConcertObjectReplicationSettings& Right)
	{
		return !(Left == Right);
	}

	/** @return Whether Left replicates less frequently than Right. */
	friend bool operator<(const FConcertObjectReplicationSettings& Left, const FConcertObjectReplicationSettings& Right)
	{
		const bool bLeftIsRealtime = Left.ReplicationMode == EConcertObjectReplicationMode::Realtime;
		const bool bRightIsRealtime = Right.ReplicationMode == EConcertObjectReplicationMode::Realtime;
		if (bLeftIsRealtime || bRightIsRealtime)
		{
			return !bLeftIsRealtime && bRightIsRealtime;
		}
		return Left.ReplicationRate < Right.ReplicationRate;
	}
	/** @return Whether Left replicates less or equally frequently than Right. */
	friend bool operator<=(const FConcertObjectReplicationSettings& Left, const FConcertObjectReplicationSettings& Right)
	{
		const bool bLeftIsRealtime = Left.ReplicationMode == EConcertObjectReplicationMode::Realtime;
		const bool bRightIsRealtime = Right.ReplicationMode == EConcertObjectReplicationMode::Realtime;
		if (bLeftIsRealtime || bRightIsRealtime)
		{
			return bRightIsRealtime;
		}
		return Left.ReplicationRate <= Right.ReplicationRate;
	}
};

/** Specifies replication frequency settings for an entire stream. */
USTRUCT()
struct FConcertStreamFrequencySettings
{
	GENERATED_BODY()

	/** Settings for all objects by default. */
	UPROPERTY()
	FConcertObjectReplicationSettings Defaults;
	
	/**
	 * Special settings to use for specific objects. Overrides DefaultSettings.
	 * May contain the same values as in DefaultSettings.
	 * Only contains entries for objects that are registered in the corresponding stream.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertObjectReplicationSettings> ObjectOverrides;

	const FConcertObjectReplicationSettings& GetSettingsFor(const FSoftObjectPath& Object) const
	{
		const FConcertObjectReplicationSettings* Settings = ObjectOverrides.Find(Object);
		return Settings ? *Settings : Defaults;
	}

	friend bool operator==(const FConcertStreamFrequencySettings& Left, const FConcertStreamFrequencySettings& Right)
	{
		return Left.Defaults == Right.Defaults
		&& Left.ObjectOverrides.OrderIndependentCompareEqual(Right.ObjectOverrides);
	}
	friend bool operator!=(const FConcertStreamFrequencySettings& Left, const FConcertStreamFrequencySettings& Right)
	{
		return !(Left == Right);
	}
};
