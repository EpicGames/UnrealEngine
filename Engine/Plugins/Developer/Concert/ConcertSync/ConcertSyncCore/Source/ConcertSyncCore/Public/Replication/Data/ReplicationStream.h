// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectReplicationMap.h"
#include "ReplicationFrequencySettings.h"
#include "ReplicationStream.generated.h"

template<typename OptionalType>
struct TOptional;

namespace UE::ConcertSyncCore
{
	enum class EReplicationStreamCloneFlags : uint8
	{
		None,
		SkipProperties = 1 << 0,
		SkipFrequency = 1 << 1
	};
	ENUM_CLASS_FLAGS(EReplicationStreamCloneFlags)
}

/**
 * Holds the base info of a replication stream: it knows the data that was registered to be sent and at what frequency.
 * 
 * This struct exists to future proof the change UE-190167, which will add UObject rule objects to streams.
 * When that happens, this struct holds all non-UObject data for the stream.
 */
USTRUCT()
struct FConcertBaseStreamInfo
{
	GENERATED_BODY()
	
	/** Unique id for this stream. */
	UPROPERTY()
	FGuid Identifier;

	/** Identifies the data that this stream will send. */
	UPROPERTY()
	FConcertObjectReplicationMap ReplicationMap;

	/**
	 * Determines how often updates are sent for objects.
	 *
	 * The purposes of registering frequency with the server communicates the sending client's intent and has these advantages:
	 * - If the server receives replicated data more often than specified here, it will combine and throttle the events forwarded to the other clients.
	 * - The receiving clients can query how often per frame to expect updates for an object
	 *
	 * On a technical level, this field is not required for replicating objects: in a hypothetical, alternate implementation, the sending client could
	 * just keep the replication frequency local to themselves and send at the rate it wants to send.
	 * This field is here mostly to allow inspection by other parties, which does add overhead (engineering and runtime) because we must support
	 * FConcertReplication_ChangeStream_Frequency requests).
	 * 
	 * With that said, the sending implementation uses the rate that is specified here to throttle the rate at which it sends update to the server.
	 * @see FObjectReplicationProcessor::ProcessObjects.
	 */
	UPROPERTY()
	FConcertStreamFrequencySettings FrequencySettings;

	/** Util for efficiently cloning stream info and optionally skipping some info. */
	FConcertBaseStreamInfo Clone(UE::ConcertSyncCore::EReplicationStreamCloneFlags Flags = UE::ConcertSyncCore::EReplicationStreamCloneFlags::None) const;
	
	friend bool operator==(const FConcertBaseStreamInfo& Left, const FConcertBaseStreamInfo& Right)
	{
		return Left.Identifier == Right.Identifier
			&& Left.ReplicationMap == Right.ReplicationMap
			&& Left.FrequencySettings == Right.FrequencySettings;
	}
	friend bool operator!=(const FConcertBaseStreamInfo& Left, const FConcertBaseStreamInfo& Right)
	{
		return !(Left == Right);
	}
};

/**
 * A replication stream a related set of data that is to be replicated. The application using this frame work (e.g. Multi User) defines what it means
 * for data to be "related".
 *
 * The stream defines:
 * - a map of objects to their replicated properties,
 * - at what frequencies the data replicates,
 * - a set of stream attributes which receiving clients can use to tell the server what data they want to receive (rules are TODO UE-190167)
 * 
 * Streams exist mostly for logical grouping and organization. It is completely acceptable for an application to only register one stream per client (Multi User does this).
 * However, there is a minor technical detail: While streams registered by the same client can replicate the same UObjects (and even overlap properties), streams send data
 * independently. The consequence is that each stream:
 * - has its own frequency counter
 * - streams data is not combined / merged (i.e. there is an overhead of the book-keeping data that is sent with each update).
 *
 * This struct only contains FConcertBaseStreamInfo for to future proof the change UE-190167, which will add rule data to streams.
 * When that happens, this struct holds all the UObject rule data for the stream while FConcertBaseStreamInfo will continue to hold non-UObject data.
 */
USTRUCT()
struct FConcertReplicationStream
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertBaseStreamInfo BaseDescription;

	friend bool operator==(const FConcertReplicationStream& Left, const FConcertReplicationStream& Right)
	{
		return Left.BaseDescription == Right.BaseDescription;
	}
	friend bool operator!=(const FConcertReplicationStream& Left, const FConcertReplicationStream& Right)
	{
		return !(Left == Right);
	}
};

inline FConcertBaseStreamInfo FConcertBaseStreamInfo::Clone(UE::ConcertSyncCore::EReplicationStreamCloneFlags Flags) const
{
	using namespace UE::ConcertSyncCore;
	FConcertBaseStreamInfo Result { Identifier };

	const TMap<FSoftObjectPath, FConcertReplicatedObjectInfo>& ReplicatedObjects = ReplicationMap.ReplicatedObjects;
	if (EnumHasAnyFlags(Flags, EReplicationStreamCloneFlags::SkipProperties)
		&& !ReplicationMap.IsEmpty())
	{
		Result.ReplicationMap.ReplicatedObjects.Reserve(ReplicatedObjects.Num());
		for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ObjectInfo : ReplicatedObjects)
		{
			Result.ReplicationMap.ReplicatedObjects.Add(ObjectInfo.Key, { ObjectInfo.Value.ClassPath });
		}
	}
	else
	{
		Result.ReplicationMap = ReplicationMap;
	}

	if (!EnumHasAnyFlags(Flags, EReplicationStreamCloneFlags::SkipFrequency))
	{
		Result.FrequencySettings = FrequencySettings;
	}
		
	return Result;
}