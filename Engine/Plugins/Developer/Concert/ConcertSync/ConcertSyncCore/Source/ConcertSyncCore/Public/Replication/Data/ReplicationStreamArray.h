// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ReplicationStream.h"
#include "ReplicationStreamArray.generated.h"

USTRUCT()
struct FConcertReplicationStreamArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FConcertReplicationStream> Streams;
};
