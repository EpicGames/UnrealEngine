// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::ConcertSyncCore
{
	/**
	 * A sequence ID is associated with every update made to a replicated object.
	 * Every time a replicated change is sent, the ID is incremented by the original data source.
	 * This is used primarily for tracing object replication performance.
	 */
	using FSequenceId = uint32;
}