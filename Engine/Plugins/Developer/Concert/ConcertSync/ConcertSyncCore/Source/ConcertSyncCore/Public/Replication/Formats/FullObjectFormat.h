// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IObjectReplicationFormat.h"

#define UE_API CONCERTSYNCCORE_API

namespace UE::ConcertSyncCore
{
	/**
	 * Full object format is a blob of serialized UObject data. 
	 *
	 * Since there no additional property information in this format, the server cannot merge data. Every update the
	 * entire blob must be sent. This format is inefficient. The only upside it that it is straight-forward to understand.
	 * 
	 * This format is used for the initial Concert Replication prototype.
	 * TODO: After the prototype phase, more efficient formats should be developed.
	 *
	 * @see FFullObjectReplicationData
	 */
	class FFullObjectFormat : public IObjectReplicationFormat
	{
	public:

		//~ Begin FFullObjectFormat Interface
		UE_API virtual TOptional<FConcertSessionSerializedPayload> CreateReplicationEvent(UObject& Object, FAllowPropertyFunc IsPropertyAllowedFunc) override;
		virtual void ClearInternalCache(TArrayView<UObject> ObjectsToClear) override { /* There is no cache: Full object format has no smart building relying on past events - it always serializes everything */ }
		UE_API virtual void CombineReplicationEvents(FConcertSessionSerializedPayload& Base, const FConcertSessionSerializedPayload& Newer) override;
		UE_API virtual void ApplyReplicationEvent(UObject& Object, const FConcertSessionSerializedPayload& Payload, const FOnPropertyVisitedFunc& OnPrePropertySerialized) override;
		//~ End FFullObjectFormat Interface
	};
}

#undef UE_API
