// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "FullObjectReplicationData.generated.h"

/**
 * Full object format is a blob of serialized UObject data.
 * @see FFullObjectFormat
 */
USTRUCT()
struct FFullObjectReplicationData
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertByteArray SerializedObjectData;
};