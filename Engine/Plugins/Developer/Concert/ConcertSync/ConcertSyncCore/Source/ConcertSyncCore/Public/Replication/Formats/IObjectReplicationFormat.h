// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Templates/Function.h"
#include "Misc/Optional.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"

class FProperty;
class UObject;
struct FArchiveSerializedPropertyChain;
struct FConcertSessionSerializedPayload;

namespace UE::ConcertSyncCore
{
	/**
	 * Abstracts the concept of a replication format for data passed into FConcertReplicationEvent::SerializedPayload.
	 *
	 * Example implementations:
	 * - full-object format that serializes all replicated properties and never does any combining (inefficient).
	 * - partial-object format that keeps track of past properties and only sends changed replicated properties.
	 */
	class IObjectReplicationFormat
	{
	public:

		/** Chain can be nullptr or non-null and empty - both imply the Property is a root property. */
		using FAllowPropertyFunc = TFunctionRef<bool(const FArchiveSerializedPropertyChain* Chain, const FProperty& Property)>;
		using FOnPropertyVisitedFunc = TFunctionRef<void(const FArchiveSerializedPropertyChain&, FProperty&)>;

		/**
		 * Produces a new payload from Object.
		 * TODO: Probably the signature of this function will be extended so you can build from a passed-in FConcertSessionSerializedPayload. Maybe for server sending to clients.
		 * 
		 * Implementations may create a diff from a previous version of the Object. This function may update the internal cache.
		 * If there is not data to apply since the last call, then this returns an empty optional.
		 *
		 * @param Object The object to serialize
		 * @param IsPropertyAllowedFunc Decides whether a given property is generally supposed to be replicated.
		 *		  Note: IObjectReplicationFormat does diffing itself - only return whether you ever want to replicate the property!
		 *
		 * @return	Data to be placed into FConcertReplicationEvent::SerializedPayload. This can be a diff from the last time CreateReplicationEvent was called.
		 *			If there is no data to apply since the last time, the optional is empty.
		 */
		virtual TOptional<FConcertSessionSerializedPayload> CreateReplicationEvent(UObject& Object, FAllowPropertyFunc IsPropertyAllowedFunc) = 0;

		/**
		 * Clears the internal cache that subclass implementations may rely on for building diffed payload version in CreateReplicationEvent.
		 * The next call to CreateReplicationEvent will serialize all data.
		 */
		virtual void ClearInternalCache(TArrayView<UObject> ObjectsToClear) = 0;

		/**
		 * Combines Base and Newer to form a single payload.
		 * Calling ApplyReplicationEvent(CombineReplicationEvents(Base, Other)) should be equivalent to calling ApplyReplicationEvent(Base) followed by ApplyReplicationEvent(Newer).
		 * 
		 * @param Base Payload produced by CreateReplicationEvent. Produced before Newer. This will be overwritten to contain the combined result.
		 * @param Newer Payload produced by CreateReplicationEvent. Produced after Base and on the same UObject.
		 */
		virtual void CombineReplicationEvents(FConcertSessionSerializedPayload& Base, const FConcertSessionSerializedPayload& Newer) = 0;

		/**
		 * Applies Payload to Object.
		 * @param Object The object to apply to
		 * @param Payload The payload to apply. Must have been created via CreateReplicationEvent.
		 * @param OnPrePropertySerialized Invoked when a property is about to be serialized. Useful for determining whether e.g. render state needs to be dirtied, etc.
		 */
		virtual void ApplyReplicationEvent(UObject& Object, const FConcertSessionSerializedPayload& Payload,
			const FOnPropertyVisitedFunc& OnPrePropertySerialized = [](const FArchiveSerializedPropertyChain&, FProperty&){}
			) = 0;

		virtual ~IObjectReplicationFormat() = default;
	};
}
