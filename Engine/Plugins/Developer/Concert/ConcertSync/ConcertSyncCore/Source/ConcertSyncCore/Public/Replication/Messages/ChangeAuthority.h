// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EReplicationResponseErrorCode.h"
#include "SyncControl.h"
#include "ChangeAuthority.generated.h"

USTRUCT()
struct FConcertStreamArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FGuid> StreamIds;
};

/**
 * Sent by clients to the server to attempt to take or release authority over properties on objects.
 * Clients must have authority over object properties before sending FConcertBatchReplicationEvent, or the server will reject the updates.
 * 
 * Multiple clients can have authority over the same objects as long as the properties do not overlap.
 * The properties are defined implicitly by providing the stream ID that will be writing the properties.
 */
USTRUCT()
struct FConcertReplication_ChangeAuthority_Request
{
	GENERATED_BODY()

	// In the future we can consider adding an option to make this all or nothing (if one object fails, the entire request does)
	// It is supposed to be a convenience and network optimization to send one big request instead of many small ones.

	/**
	 * Objects the client requests to take authority over.
	 * Mapping of object to stream identifiers the client has previously registered.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertStreamArray> TakeAuthority;

	/**
	 * Objects the client no longer needs authority over.
	 * Mapping of object to stream identifiers the client has previously registered.
	 * 
	 * The request will not fail if it contains objects it has no authority over.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertStreamArray> ReleaseAuthority;

	bool IsEmpty() const { return TakeAuthority.IsEmpty() && ReleaseAuthority.IsEmpty(); }
};

USTRUCT()
struct FConcertReplication_ChangeAuthority_Response
{
	GENERATED_BODY()

	/** Concert's custom requests are default constructed when they timeout. Server always sets this to Handled when processed. */
	UPROPERTY()
	EReplicationResponseErrorCode ErrorCode = EReplicationResponseErrorCode::Timeout;
	
	/**
	 * Objects streams the client did not receive authority over.
	 * The client is implied to have authority over all other request objects.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertStreamArray> RejectedObjects;

	/**
	 * Contains the new sync control for objects you requested authority over; objects you released authority over are not contained because they can
	 * be implicitly determined using FSyncControlState::AppendChanges.
	 * 
	 * @see FSyncControlState::Aggregate.
	 * @see FConcertReplication_ChangeSyncControl
	 */
	UPROPERTY()
	FConcertReplication_ChangeSyncControl SyncControl;
};