// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChangeAuthority.h"
#include "ChangeStream.h"
#include "ChangeClientEvent.generated.h"

/** A reason why a FConcertReplication_ChangeClientEvent is sent. */
UENUM()
enum class EConcertReplicationChangeClientReason : uint8
{
	/** The code is unknown */
	Unknown,
	
	/** A FConcertReplication_PutState_Request edited this client. */
	PutRequest
};

/** Describes what aspects about a client have changed. */
USTRUCT()
struct FConcertReplication_ClientChangeData
{
	GENERATED_BODY()
	
	/** The change made to the client's streams. */
	UPROPERTY()
	FConcertReplication_ChangeStream_Request StreamChange;

	/** The change made to the client's authority. */
	UPROPERTY()
	FConcertReplication_ChangeAuthority_Request AuthorityChange;

	/** The change made to the client's sync control in response to the above changes. */
	UPROPERTY()
	FConcertReplication_ChangeSyncControl SyncControlChange;
};

/**
 * Sent by server to notify a client that their stream content and / or authority has been changed by an external entity,
 * i.e. when the change was not initiated by the client itself.
 */
USTRUCT()
struct FConcertReplication_ChangeClientEvent
{
	GENERATED_BODY()

	/** The reason for this event */
	UPROPERTY()
	EConcertReplicationChangeClientReason Reason = EConcertReplicationChangeClientReason::Unknown;

	/** Info about what has changed. */
	UPROPERTY()
	FConcertReplication_ClientChangeData ChangeData;
};