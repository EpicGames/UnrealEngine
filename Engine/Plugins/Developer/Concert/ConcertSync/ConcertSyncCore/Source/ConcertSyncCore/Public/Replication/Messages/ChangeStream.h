// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EReplicationResponseErrorCode.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Data/ReplicationStream.h"

#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "ChangeStream.generated.h"

class FOutputDevice;

struct FConcertReplicatedObjectInfo;

/** A request to add a new object to a stream or overwrite a pre-existing object's properties / class. */
USTRUCT()
struct FConcertReplication_ChangeStream_PutObject
{
	GENERATED_BODY()

	// TODO UE-201166: Add enum flags here that describe how Properties and ClassPath are to be interpreted.
	
	/**
	 * The property selection the object should have.
	 * Objects must always have a non-empty selection: use FConcertChangeStream_Request::ObjectsToRemove to remove objects.
	 *
	 * Any request that would leave an object's property selection empty will result in failure.
	 */
	UPROPERTY()
	FConcertPropertySelection Properties;

	/**
	 * If Object is pre-existing, this property is optional. If different than None, this will change the class
	 * If a new object is added, this property is mandatory. Not doing so will fail the request.
	 *
	 * Not specifying ClassPath nor Properties is an error: use FConcertChangeStream_Request::ObjectsToRemove to remove objects.
	 */
	UPROPERTY()
	FSoftClassPath ClassPath;

	friend bool operator==(const FConcertReplication_ChangeStream_PutObject& Left, const FConcertReplication_ChangeStream_PutObject& Right)
	{
		return Left.Properties == Right.Properties && Left.ClassPath == Right.ClassPath;
	}

	friend bool operator!=(const FConcertReplication_ChangeStream_PutObject& Left, const FConcertReplication_ChangeStream_PutObject& Right)
	{
		return !(Left == Right);
	}

	// Intention: Ideally code dealing with PutObject requests uses these constructors / factory functions.
	// If a property is added to FConcertReplicatedObjectInfo, only the below code needs to be updated.

	/** @return The PutObject request if New contained sufficient info - empty otherwise */
	CONCERTSYNCCORE_API static TOptional<FConcertReplication_ChangeStream_PutObject> MakeFromInfo(const FConcertReplicatedObjectInfo& New);
	/** @return The PutObject request if changing Base to Desired contained sufficient info - empty otherwise */
	CONCERTSYNCCORE_API static TOptional<FConcertReplication_ChangeStream_PutObject> MakeFromChange(const FConcertReplicatedObjectInfo& Base, const FConcertReplicatedObjectInfo& Desired);

	/** Creates a new object info if there is sufficient data (all fields must be set for this). */
	CONCERTSYNCCORE_API TOptional<FConcertReplicatedObjectInfo> MakeObjectInfoIfValid() const;
};

UENUM(Flags)
enum class EConcertReplicationChangeFrequencyFlags : uint8
{
	None = 0,
	/** FConcertReplication_ChangeStream_Frequency::NewDefaults contains the new defaults to set. */
	SetDefaults = 1 << 0,
};
ENUM_CLASS_FLAGS(EConcertReplicationChangeFrequencyFlags);

/** Changes to set to make to the stream's replication frequency. */
USTRUCT()
struct FConcertReplication_ChangeStream_Frequency
{
	GENERATED_BODY()

	/**
	 * Replaces the entire overrides with this content.
	 * If a specified object is not specified in the associated stream, this request DOES fail.
	 * 
	 * Usually you want to specify either OverridesToPut, or OverridesToPut & OverridesToRemove but not both.
	 * A put is useful for situations where you don't care what objects have been overriden thus far and want to straight out replace them all.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertObjectReplicationSettings> OverridesToPut;

	/**
	 * Removes the specified overrides.
	 * If a specified object is not specified in the associated stream, this request DOES NOT fail.
	 * 
	 * Executed after OverridesToPut (but you should either use OverridesToPut, or OverridesToRemove & OverridesToAdd).
	 */
	UPROPERTY()
	TSet<FSoftObjectPath> OverridesToRemove;
	
	/**
	 * Adds the specified overrides.
	 * If a specified object is not specified in the associated stream, this request DOES fail.
	 * 
	 * Executed after OverridesToAdd (but you should either use OverridesToPut, or OverridesToRemove & OverridesToAdd).
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertObjectReplicationSettings> OverridesToAdd;

	/** Only applied if Flags contains the SetDefaults flag. */
	UPROPERTY()
	FConcertObjectReplicationSettings NewDefaults;
	
	UPROPERTY()
	EConcertReplicationChangeFrequencyFlags Flags = EConcertReplicationChangeFrequencyFlags::None;
};

/**
 * Let's a client change its streams owned on the server.
 * 
 * This request is processed atomically: it either succeeds completely or fails completely.
 * If any sub-change causes a failure, the failure reason will be returned in the response and the client must make a new request.
 */
USTRUCT()
struct FConcertReplication_ChangeStream_Request
{
	GENERATED_BODY()

	/**
	 * Removes objects from pre-existing streams.
	 * Supplying an object that is not in the specified stream does not cause failure (but is nonsensical).
	 * 
	 * If a stream has no registered objects after this operation, it is automatically removed.
	 * If the requesting client has authority over these objects, authority is removed.
	 */
	UPROPERTY()
	TSet<FConcertObjectInStreamID> ObjectsToRemove;

	/**
	 * Adds new or modifies preexisting object definitions in pre-existing streams.
	 *
	 * If the requesting client and a different client have authority over the same object, you will get a conflict
	 * if the different client already has authority over one of the properties you're adding here.
	 * @see FConcertChangeStream_Response::AuthorityConflicts for some examples of conflicts.
	 *
	 * If the key identifies a stream that does not exist, the request will fail.
	 *
	 * TODO UE-201167: Make the request no longer fail
	 * If the key identifies a stream that is in StreamsToAdd, the request will fail to avoid allowing the construction of
	 * ambiguous requests, e.g. both StreamsToAdd and ObjectsToPut containing object Foo but with different property selections.
	 */
	UPROPERTY()
	TMap<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject> ObjectsToPut;

	/**
	 * Changes the replication frequency settings for a stream.
	 * 
	 * If a specified stream does not exist, the request fails.
	 * Specifying a request that changes nothing does not result in any failure.
	 * 
	 * TODO UE-201167: Change StreamsToAdd such that no ambiguous requests are even possible and update this doc's wording.
	 * If the key may identify a stream that is in StreamsToAdd, in which case the frequency is what is specified in FrequencyChanges.
	 */
	UPROPERTY()
	TMap<FGuid, FConcertReplication_ChangeStream_Frequency> FrequencyChanges;
	
	// TODO UE-201167: Change StreamsToAdd so you just give FGuid of new streams and specify properties in ObjectsToPut and FrequencyChanges > easier API usage.
	
	/**
	 * New streams to add to the server.
	 * Fails if any ID overlaps with a pre-existing one.
	 *
	 * It is valid to have the same stream in StreamsToRemove: StreamsToAdd is applied after StreamsToRemove resulting
	 * in the stream's content being replaced.
	 */
	UPROPERTY()
	TArray<FConcertReplicationStream> StreamsToAdd;

	/**
	 * Streams to remove from the server.
	 * Supplying a stream that does not exist does not cause failure (but is nonsensical).
	 * 
	 * If the requesting client has authority over any of the objects contained in the stream, authority is removed.
	 * 
	 * It is valid to have the same stream in StreamsToAdd: StreamsToAdd is applied after StreamsToRemove resulting
	 * in the stream's content being replaced.
	 */
	UPROPERTY()
	TSet<FGuid> StreamsToRemove;

	bool IsEmpty() const { return ObjectsToRemove.IsEmpty() && ObjectsToPut.IsEmpty() && FrequencyChanges.IsEmpty() && StreamsToAdd.IsEmpty() && StreamsToRemove.IsEmpty(); }

	friend bool operator==(const FConcertReplication_ChangeStream_Request& Left, const FConcertReplication_ChangeStream_Request& Right)
	{
		const auto OrderIndependentEquals = [](const auto& Left, const auto& Right){ return Left.Num() == Right.Num() && Left.Includes(Right); };
		return OrderIndependentEquals(Left.ObjectsToRemove, Right.ObjectsToRemove)
			&& OrderIndependentEquals(Left.StreamsToRemove, Right.StreamsToRemove)
			&& Left.ObjectsToPut.OrderIndependentCompareEqual(Right.ObjectsToPut)
			&& Left.StreamsToAdd == Right.StreamsToAdd;
	}
	friend bool operator!=(const FConcertReplication_ChangeStream_Request& Lhs, const FConcertReplication_ChangeStream_Request& RHS)
	{
		return !(Lhs == RHS);
	}
};

UENUM()
enum class EConcertPutObjectErrorCode : uint8
{
	/** Stream that ObjectsToPut referenced was not registered on the server. */
	UnresolvedStream,
	/**
	 * Either PutObject contained no data to update with (ensure either ClassPath or Properties is set),
	 * or it tried to create a new object with insufficient data (make sure ClassPath and Properties are both specified).
	 */
	MissingData,

	/** Not an actual parameter. Make sure it's always last. */
	Count
};

UENUM()
enum class EConcertChangeObjectFrequencyErrorCode : uint8
{
	/** The object for which the frequency was being changed was not registered. */
	NotRegistered,
	/** The replication rate parameter was rejected (it cannot be 0). */
	InvalidReplicationRate,

	/** Not an actual parameter. Make sure it's always last. */
	Count
};

UENUM()
enum class EConcertChangeStreamFrequencyErrorCode : uint8
{
	/** The stream was not registered */
	UnknownStream,
	/** The replication rate parameter was rejected (it cannot be 0). */
	InvalidReplicationRate,

	/** Not an actual parameter. Make sure it's always last. */
	Count
};

USTRUCT()
struct FConcertReplication_ChangeStream_FrequencyResponse
{
	GENERATED_BODY()

	/** Streams that could not have their frequencies overriden. */
	UPROPERTY()
	TMap<FConcertObjectInStreamID, EConcertChangeObjectFrequencyErrorCode> OverrideFailures;

	/** Streams that could not have their frequency defaults overriden. */
	UPROPERTY()
	TMap<FGuid, EConcertChangeStreamFrequencyErrorCode> DefaultFailures;
	
	bool IsSuccess() const { return OverrideFailures.IsEmpty() && DefaultFailures.IsEmpty(); }
	bool IsFailure() const { return !IsSuccess(); }
};

/**
 * Contains information about why a request failed. This info could be parsed and displayed to the end user as error.
 * If there is even just one error, the entire request fails and no changes are made server-side.
 */
USTRUCT()
struct FConcertReplication_ChangeStream_Response
{
	GENERATED_BODY()

	/** Concert's custom requests are default constructed when they timeout. Server always sets this to Handled when processed. */
	UPROPERTY()
	EReplicationResponseErrorCode ErrorCode = EReplicationResponseErrorCode::Timeout;
	
	/**
	 * Reports dynamic authority errors with ObjectsToPut:
	 * Changing an object over which a client already has authority can yield unresolvable conflicts for which the entire FConcertChangeStream_Request is rejected.
	 *
	 * Let client R be the requester and client A be another client.
	 * Example 1: Overlapping authority on different clients > Conflict
	 * - R has authority over Foo's relative rotation property in stream S.
	 * - A has authority over actor Foo's relative location property in some stream.
	 * - R requests S to include Foo's relative location property: this conflicts with client A's authority.
	 *
	 * Example 2: Overlapping, properties on same client > No conflict
	 * There is no conflict if another stream owned by the requester already has authority over a property you're adding to a different stream.
	 * Streams of the same client may overlap properties. While this may not make much sense and actually be performance degrading, it would result in no logical errors so it is allowed.
	 * Example: R has two streams S1 and S2. S1 contains the transform properties and S2 does not. Both S1 and S2 have authority over object Foo. It is legal to request S2 to contain the transform properties.
	 */
	UPROPERTY()
	TMap<FConcertObjectInStreamID, FConcertReplicatedObjectId> AuthorityConflicts;

	/** Reports semantic errors with ObjectsToPut. */
	UPROPERTY()
	TMap<FConcertObjectInStreamID, EConcertPutObjectErrorCode> ObjectsToPutSemanticErrors;
	
	/** Streams that were in StreamsToAdd but that were not created (stream was already registered, or was empty). */
	UPROPERTY()
	TSet<FGuid> FailedStreamCreation;

	/** Errors updating the frequency */
	UPROPERTY()
	FConcertReplication_ChangeStream_FrequencyResponse FrequencyErrors;
	
	bool IsSuccess() const
	{
		return ErrorCode == EReplicationResponseErrorCode::Handled
			&& AuthorityConflicts.IsEmpty()
			&& ObjectsToPutSemanticErrors.IsEmpty()
			&& FailedStreamCreation.IsEmpty()
			&& FrequencyErrors.IsSuccess();
	}
	bool IsFailure() const { return !IsSuccess(); }

	bool WasObjectPutSuccessful(const FConcertObjectInStreamID& Object) const { return !AuthorityConflicts.Contains(Object) && !ObjectsToPutSemanticErrors.Contains(Object); }
};