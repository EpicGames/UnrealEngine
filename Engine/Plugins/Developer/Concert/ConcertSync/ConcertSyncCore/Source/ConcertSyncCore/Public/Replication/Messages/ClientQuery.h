// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EReplicationResponseErrorCode.h"
#include "Replication/Data/ClientQueriedInfo.h"
#include "ClientQuery.generated.h"

/** Flags to affect FConcertQueryClientStreams_Request */
UENUM()
enum class EConcertQueryClientStreamFlags : uint8
{
	None = 0,

	/** If set, FReplicationClientQueriedInfo::Streams will be empty. */
	SkipStreamInfo = 1 << 0,

	/**
	 * Optimization. The returned streams will not contain the properties that are sent.
	 * Set this if the request only cares for the objects being replicated but not its content.
	 * Only has an effect if SkipStreamInfo is not set.
	 */
	SkipProperties = 1 << 1,
	
	/** If set, FReplicationClientQueriedInfo::Authority will be empty. */
	SkipAuthority = 1 << 2,

	/**
	 * Optimization. Ignored if SkipStreamInfo is set.
	 * If set, FReplicationClientQueriedInfo::Streams.FrequencySettings will not be set.
	 */
	SkipFrequency = 1 << 3,
};
ENUM_CLASS_FLAGS(EConcertQueryClientStreamFlags)

/**
 * Allows a client query about another client's registered streams.
 *
 * Currently, clients must continuously poll to detect when a client changes this data.
 * TODO: Add an event with which clients can be notified when another client changes their profile.
 * TODO: Alternative to an event, we can add a version int which is incremented every time the info changes.
 */
USTRUCT()
struct FConcertReplication_QueryReplicationInfo_Request
{
	GENERATED_BODY()

	/** The clients to query about */
	UPROPERTY()
	TSet<FGuid> ClientEndpointIds;

	UPROPERTY()
	EConcertQueryClientStreamFlags QueryFlags = EConcertQueryClientStreamFlags::None;
};

USTRUCT()
struct FConcertReplication_QueryReplicationInfo_Response 
{
	GENERATED_BODY()
	
	/** Concert's custom requests are default constructed when they timeout. Server always sets this to Handled when processed. */
	UPROPERTY()
	EReplicationResponseErrorCode ErrorCode = EReplicationResponseErrorCode::Timeout;
	
	/**
	 * Binds client participating in the replication session to their info.
	 * 
	 * @note This contains only the client endpoint IDs that are in the replication session at the time of processing the request.
	 * That means not every endpoint ID that was in the request's ClientEndpointIds will necessarily be in this result. For example,
	 * if client A sends a FConcertQueryReplicationInfo_Request while client B disconnects, client B may not show up in ClientInfo.
	 * 
	 * Key: Client endpoint ID.
	 * Value: Info about the client.
	 */
	UPROPERTY()
	TMap<FGuid, FConcertQueriedClientInfo> ClientInfo;
};