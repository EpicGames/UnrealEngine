// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EReplicationResponseErrorCode.generated.h"

/**
 * Added as field to all replication response data.
 * Used to detect timeouts or other delivery failures.
 * 
 * Custom requests and responses are considered failed if they are default constructed.
 * The default constructed responses should be EReplicationResponseErrorCode::Timout by default.
 */
UENUM()
enum class EReplicationResponseErrorCode : uint8
{
	/** Value to set when default constructed. Indicates responses not received. */
	Timeout,
	/** Set when request was handled and there is valid response data */
	Handled
};