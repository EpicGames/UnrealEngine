// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SyncControl.h"
#include "Replication/Data/ReplicationStream.h"
#include "Handshake.generated.h"

class FText;

UENUM()
enum class EJoinReplicationErrorCode : uint8
{
	Success = 0,
	/** Indicates generic error obtaining a response from the server. Responses are default constructed if an error occurs. This is the default value. */
	NetworkError = 1,

	// Client-side error codes:
	
	/** Cancelled while initialization was in progress. Most likely LeaveReplicationSession was called while handshake was in progress. */
	Cancelled = 2,
	/** Another request is already in progress. */
	AlreadyInProgress = 3,

	// Server error codes:

	// Session:
	/**
	 * Client is not in any session but has requested to join a replication session.
	 * This case should never happen under normal conditions e.g. when using Multi User Browser, but malicious users could make invalid requests.
	 */
	NotInAnyConcertSession = 4,
	/** Client has already joined a replication session. Send FConcertLeaveReplicationSessionEvent first. */
	AlreadyInSession = 5,

	// Invalid input:
	/** One of the specified classes was null or otherwise invalid */
	InvalidClass = 6,
	/** Your input contained the same stream id twice. */
	DuplicateStreamId = 7,
	/** Failed to unpack the stream, likely because one of the attribute classes could not be resolved. */
	FailedToUnpackStream = 8,
	
	MaxPlusOne,
	/** The maximum possible entry value */
	Max = MaxPlusOne - 1
};

namespace UE::ConcertSyncCore::Replication
{
	CONCERTSYNCCORE_API FString LexJoinErrorCode(EJoinReplicationErrorCode ErrorCode);
}

/**
 * Sent by client to request joining a replication session.
 * This is sent by clients want to send or receive replicated data, or both.
 */
USTRUCT()
struct FConcertReplication_Join_Request
{
	GENERATED_BODY()

	/** The data the client offers to send. */
	UPROPERTY()
	TArray<FConcertReplicationStream> Streams;
};

/**
 * Sent by server to accept the previously sent replication session.
 */
USTRUCT()
struct FConcertReplication_Join_Response
{
	GENERATED_BODY()

	/**
	 * Anything other than Success indicates failure.
	 * 
	 * Remember that Concert default constructs a response when it fails to obtain a response.
	 * Hence NetworkError means timeout, etc.
	 */
	UPROPERTY()
	EJoinReplicationErrorCode JoinErrorCode = EJoinReplicationErrorCode::NetworkError;

	/** More information about ErrorCode to help the user resolve the issue. */
	UPROPERTY()
	FString DetailedErrorMessage;

	/** This is the initial sync control this client has */
	UPROPERTY()
	FConcertReplication_ChangeSyncControl SyncControl;
};

/** Sent by client to notify server that no more replication data should be sent. */
USTRUCT()
struct FConcertReplication_LeaveEvent
{
	GENERATED_BODY()
};