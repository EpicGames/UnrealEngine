// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EReplicationResponseErrorCode.h"
#include "SyncControl.h"

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Muting.generated.h"

UENUM()
enum class EConcertReplicationMuteOption : uint8
{
	/** Only the specified object is affected; subobjects are not. */
	OnlyObject,
	/** The setting applies for all subobjects of the object as well. If subobjects are added in the future, the setting will apply to them as well. */
	ObjectAndSubobjects = 1 << 0,
};

namespace UE::ConcertSyncCore
{
	/** @return Whether subobjects are affected by this Option. Used for future proofing (in case new entries are added to the enum). */
	inline bool AffectSubobjects(EConcertReplicationMuteOption Option) { return Option == EConcertReplicationMuteOption::ObjectAndSubobjects; }
}

/** Describes how an object is to be muted. */
USTRUCT()
struct FConcertReplication_ObjectMuteSetting
{
	GENERATED_BODY()

	/** Modify the default behavior of the mute operation */
	UPROPERTY()
	EConcertReplicationMuteOption Flags = EConcertReplicationMuteOption::ObjectAndSubobjects;

	friend bool operator==(const FConcertReplication_ObjectMuteSetting& Left, const FConcertReplication_ObjectMuteSetting& Right) = default;
};

UENUM(Flags)
enum class EConcertReplicationMuteRequestFlags : uint8
{
	None,

	/** Before the request is applied, all mute state is reset. This means that ObjectsToUnmute is effectively ignored and the mute state is replaced with what is saved in ObjectsToMute. */
	ClearMuteState = 1 << 0,
};
ENUM_CLASS_FLAGS(EConcertReplicationMuteRequestFlags);

/**
 * A request to globally pause / resume replication of objects.
 * Muted objects will not be replicated to any clients.
 * For security reasons, muting is only available in sessions with the EConcertSyncSessionFlags::ShouldAllowGlobalMuting flag set.
 *
 * If a muted objects is unregistered (i.e. by a stream change request), the mute state is also removed.
 * That the object is later re-registered, it will not be muted.
 *
 * Requests are processed atomically: it is completed either completely or not at all.
 * The following operations are invalid:
 * - Unmuting / muting an object without the IncludeSubobjects flag if that object is not registered in any stream.
 * - Unmuting / muting an object with the IncludeSubobjects flag if that object is not registered in any stream, and that object has no subobjects registered in any streams.
 * The reason for making these operations invalid is to prevent memory leaks on the server (e.g. client unregisters an object and forgets to remove the mute state, too).
 * The following operations are valid:
 * - Unmuting an object that is already unmuted (no-op)
 * - Muting an object that is already muted (no-op); changing the EConcertReplicationMuteFlags is not a no-op.
 *
 * Sync control:
 * This request may update sync control. The following rules apply:
 * - This request may cause FConcertReplication_ChangeSyncControl events to be sent to OTHER clients.
 * - This request will NOT cause any FConcertReplication_ChangeSyncControl event to be sent to the requesting client:
 *		- When muting, the requesting client is expected to infer that sync control is lost for the muted objects.
 *		- When unmuting, the server fills FConcertReplication_ChangeMuteState_Response::SyncControl with the object the requesting client has received sync control over.
 */
USTRUCT()
struct FConcertReplication_ChangeMuteState_Request
{
	GENERATED_BODY()

	/** Flags that modify the request's behavior. */
	UPROPERTY()
	EConcertReplicationMuteRequestFlags Flags = EConcertReplicationMuteRequestFlags::None;
	
	/** The objects to explicitly mute. */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> ObjectsToMute;

	/**
	 * The objects to explicitly unmute.
	 * 
	 * If a specified object is implicitly muted, i.e. one of its outers is muted with the EConcertReplicationMuteFlags::ObjectAndSubobjects flag,
	 * this will allow it to replicate again. If you specify the EConcertReplicationMuteFlags::ObjectAndSubobjects flag, then its subobjects will
	 * also be allowed to be replicated again.
	 *
	 * If a specified object is explicitly muted, then it and all the subobjects that were implicitly muted because of it are unmuted;
	 * in this case, it does not matter whether EConcertReplicationMuteFlags::ObjectAndSubobjects flag is set.
	 *
	 * If Flags specifies ClearMuteState and ObjectsToUnmute is non-empty, the request is rejected.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> ObjectsToUnmute;

	friend bool operator==(const FConcertReplication_ChangeMuteState_Request& Left, const FConcertReplication_ChangeMuteState_Request& Right)
	{
		return Left.ObjectsToMute.OrderIndependentCompareEqual(Right.ObjectsToMute) && Left.ObjectsToUnmute.OrderIndependentCompareEqual(Right.ObjectsToUnmute);
	}
	friend bool operator!=(const FConcertReplication_ChangeMuteState_Request&, const FConcertReplication_ChangeMuteState_Request&) = default;

	/** @return Whether this requests makes no changes */
	bool IsEmpty() const { return ObjectsToMute.IsEmpty() && ObjectsToUnmute.IsEmpty(); }
};

/** Result of a FConcertReplication_ChangeMuteState_Request. */
UENUM()
enum class EConcertReplicationMuteErrorCode : uint8
{
	/**
	 * Value to set when default constructed (Concert does this to responses when they timeout or the request in invalid).
	 * Happens when timed out or because EConcertSyncSessionFlags::ShouldAllowGlobalMuting is not set.
	 */
	Timeout,

	/** Changes were applied. */
	Accepted,
	
	/** No change were made. The request was malformed. */
	Rejected
};

/** If the ErrorCode != Accepted, then no changes were made on the server. */
USTRUCT()
struct FConcertReplication_ChangeMuteState_Response
{
	GENERATED_BODY()

	UPROPERTY()
	EConcertReplicationMuteErrorCode ErrorCode = EConcertReplicationMuteErrorCode::Timeout;

	/** The objects that caused the request to be rejected. Only valid if EConcertReplicationMuteErrorCode::Rejected. */
	UPROPERTY()
	TSet<FSoftObjectPath> RejectionReasons;

	/**
	 * If the request unmuted objects, this contains the objects that the requester gained sync control over.
	 * 
	 * @note When muting, the requesting client is expected to infer that sync control is lost for the muted objects.
	 * Hence, this will not contain objects the requester lost sync control over.
	 */
	UPROPERTY()
	FConcertReplication_ChangeSyncControl SyncControl;

	bool IsSuccess() const { return ErrorCode == EConcertReplicationMuteErrorCode::Accepted && ensure(RejectionReasons.IsEmpty()); }
	bool IsFailure() const { return !IsSuccess(); }
};

/** Queries the effective mute state for.*/
USTRUCT()
struct FConcertReplication_QueryMuteState_Request
{
	GENERATED_BODY()

	/** Specifies the objects for which to get the mute states. If left empty, get the states of all objects. */
	UPROPERTY()
	TSet<FSoftObjectPath> QueriedObjects;

	bool WantsAllObjects() const { return QueriedObjects.IsEmpty(); }
};

USTRUCT()
struct FConcertReplication_QueryMuteState_Response
{
	GENERATED_BODY()

	UPROPERTY()
	EReplicationResponseErrorCode ErrorCode = EReplicationResponseErrorCode::Timeout;

	/**
	 * Objects that were explicitly muted, i.e. specified with FConcertReplication_ChangeMuteState_Request::ObjectsToMute.
	 * */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> ExplicitlyMutedObjects;

	/**
	 * After an outer object is muted with EConcertReplicationMuteFlags::ObjectAndSubobjects, this contains the objects that were explicitly unmuted,
	 * i.e. specified with FConcertReplication_ChangeMuteState_Request::ObjectsToUnmute.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> ExplicitlyUnmutedObjects;

	/**
	 * Objects that were implicitly muted, i.e. some their outers are explicitly muted with EConcertReplicationMuteFlags::ObjectAndSubobjects.
	 */
	UPROPERTY()
	TSet<FSoftObjectPath> ImplicitlyMutedObjects;

	/** Objects that were implicitly unmuted, i.e. 1. a parent is muted and 2. a child of parent is unmuted with EConcertReplicationMuteFlags::ObjectAndSubobjects */
	UPROPERTY()
	TSet<FSoftObjectPath> ImplicitlyUnmutedObjects;

	bool IsSuccess() const { return ErrorCode == EReplicationResponseErrorCode::Handled; }
	bool IsFailure() const { return !IsSuccess(); }
	bool IsEmpty() const { return ExplicitlyMutedObjects.IsEmpty() && ExplicitlyUnmutedObjects.IsEmpty() && ImplicitlyMutedObjects.IsEmpty(); }
};