// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "Misc/Optional.h"
#include "ObjectReplication.generated.h"

class FOutputDevice;

/** Contains data to be applied to a replicated object. */
USTRUCT()
struct FConcertReplication_ObjectReplicationEvent
{
	GENERATED_BODY()

	/**
	 * The object to serialize into.
	 * TODO: To reduce network load, replace this with an int32 that maps into FObjectToIdTable (custom) / UPackageMap (engine)  shared by all clients and the server.
	 */
	UPROPERTY()
	FSoftObjectPath ReplicatedObject;
	
	/** Incremented every time replication data is sent out. Used for performance tracing. */
	UPROPERTY()
	int32 ReplicationSequenceId = -1;
	
	/** Contains another struct as payload, such as FConcertFullObjectReplicationData. */
	UPROPERTY()
	FConcertSessionSerializedPayload SerializedPayload;
};

/** Contains data produced by a specific replication stream. */
USTRUCT()
struct FConcertReplication_StreamReplicationEvent
{
	GENERATED_BODY()

	/** The stream that produced this data. */
	UPROPERTY()
	FGuid StreamId; // TODO DP UE-193261: This field is only relevant to the server and useless for receiving clients.

	/** The objects replicated by this event */
	UPROPERTY()
	TArray<FConcertReplication_ObjectReplicationEvent> ReplicatedObjects;

	FConcertReplication_StreamReplicationEvent() = default;
	explicit FConcertReplication_StreamReplicationEvent(const FGuid& StreamId)
		: StreamId(StreamId)
	{}
};

/**
 * Contains multiple objects to be replicated.
 * Sent from
 *  - Authoritative client to server
 *  - Server to clients. Every event may be customized for each client (clients can choose to ignore object updates).
 *
 *  TODO DP UE-193261: This should be split up into FConcertBatchReplicationEvent_ToServer and FConcertBatchReplicationEvent_ToClient; clients do not need FConcertStreamReplicationEvent::StreamId. 
 */
USTRUCT()
struct FConcertReplication_BatchReplicationEvent 
{
	GENERATED_BODY()
	
	/** The objects replicated by this event */
	UPROPERTY()
	TArray<FConcertReplication_StreamReplicationEvent> Streams;
};