// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Muting.h"
#include "Replication/Data/AuthorityConflict.h"
#include "Replication/Data/ReplicationStreamArray.h"
#include "PutState.generated.h"

UENUM(Flags) 
enum class EConcertReplicationPutStateFlags : uint8
{
	None,

	/** If another client has disconnected between the time the request was sent and received by the client, do not fail the request. */
	SkipDisconnectedClients = 1 << 0,

	Default = SkipDisconnectedClients
};
ENUM_CLASS_FLAGS(EConcertReplicationPutStateFlags);

/**
 * Requests that sets the state of replication atomically.
 * It can be used to change the streams and / or authority of multiple clients together with the mute state in a single request.
 * This is useful i.e. for setting the session to a preset state.
 * 
 * Upon successful execution, the server will send a FConcertReplication_ChangeClientEvent notification to all affected clients except the requesting
 * client to update them about the changed information; the requesting client receives only sync control changes (since they know the rest they changed).
 * 
 * Upon failure, it is guaranteed no changes have taken place.
 * The request fails whenever any of the following occurs:
 *	- applying the request would generate any authority conflict.
 *	For example, a request that wants client 1 and 2 to both replicate the same properties of the same object would fail.
 *	- a client leaves replication before the request is processed and the SkipDisconnectedClients is not set.
 *
 * This request is only available if EConcertSyncSessionFlags::ShouldEnableRemoteEditing.
 */
USTRUCT()
struct FConcertReplication_PutState_Request
{
	GENERATED_BODY()

	/** Additional flags that alter the behavior of the request. */
	UPROPERTY()
	EConcertReplicationPutStateFlags Flags = EConcertReplicationPutStateFlags::Default;

	/** Maps a client's endpoint ID to the stream content it should have. */
	UPROPERTY()
	TMap<FGuid, FConcertReplicationStreamArray> NewStreams;

	/** Maps a client's endpoint ID to new authority it should have. */
	UPROPERTY()
	TMap<FGuid, FConcertObjectInStreamArray> NewAuthorityState;

	/** Mute state to apply after NewStreams and NewAuthorityState has been applied. */
	UPROPERTY()
	FConcertReplication_ChangeMuteState_Request MuteChange;
};

/**
 * Indicates success or failure of the FConcertReplication_PutState_Request.
 * 
 * While a request may have multiple errors, the errors are returned in declaration order of this enum.
 * Hence, the response code is the first thing that fails when reading top to bottom.
 */
UENUM()
enum class EConcertReplicationPutStateResponseCode : uint8
{
	/** The request timed out. */
	Timeout,
	/** The request was executed successfully. */
	Success,

	/**
	 * Either:
	 * - Session does not allow remote editing (EConcertSyncSessionFlags::ShouldEnableRemoteEditing is disabled).
	 * - Session does not allow muting (EConcertSyncSessionFlags::ShouldAllowGlobalMuting is disabled) and a mute request was made.
	 */
	FeatureDisabled,
	
	/**
	 * No changes have been made.
	 * 
	 * Only returned if the flag EConcertReplicationChangeClientsFlags:: SkipDisconnectedClients was not set and
	 * FConcertReplication_PutState_Request::StreamChanges or FConcertReplication_PutState_Request::AuthorityChanges referenced a client
	 * endpoint not in the session or that has not joined replication.
	 * 
	 * FConcertReplication_PutState_Response::UnknownEndpoints contains the unknown endpoints.
	 */
	ClientUnknown,

	/**
	 * No changes have been made. 
	 * The StreamChanges was produced an error incorrect, e.g. if a stream change references an unknown stream
	 * or if a mute request references an object unknown to the server.
	 */
	StreamError,
	
	/**
	 * No changes have been made. 
	 * The request failed because the AuthorityChanges portion would generate a conflict with other clients if applied.
	 */
	AuthorityConflict,

	/**
	 * No changes have been made.
	 * The request failed because 
	 */
	MuteError
};

USTRUCT()
struct FConcertReplication_PutState_Response
{
	GENERATED_BODY()

	UPROPERTY()
	EConcertReplicationPutStateResponseCode ResponseCode = EConcertReplicationPutStateResponseCode::Timeout;

	/**
	 * Maps a key from AuthorityChanges to the errors it had.
	 * Set if ResponseCode == EConcertReplicationChangeClientsResponseCode::AuthorityConflict.
	 */
	UPROPERTY()
	TMap<FGuid, FConcertAuthorityConflictArray> AuthorityChangeConflicts;

	/** Sets if ResponseCode == EConcertReplicationChangeClientsResponseCode::ClientUnknown. */
	UPROPERTY()
	TSet<FGuid> UnknownEndpoints;

	/**
	 * The sync control changes that happened to the requesting client.
	 *
	 * Generally, this does not contain changes that the client can predict locally, i.e.
	 * 
	 * - Retaining sync control.
	 *	If the requesting client had sync control for an object (before the request) and the request does not modify it, that object is not contained in SyncControl.
	 *	For example, if the client has sync control over Foo, and NewStreams & NewAuthorityState still contain Foo, Foo is not listed in SyncControl (since the client can predict the change).
	 *	
	 * - Muting:
	 *	If the requesting client had sync control for an object (before the request) and the request mutes it, that object is not contained in SyncControl.
	 *	For example, if the client has sync control over Foo, and MuteChanges mutes Foo, Foo is not listed in SyncControl (since the client can predict the change).
	 *	
	 */
	UPROPERTY()
	FConcertReplication_ChangeSyncControl SyncControl;

	bool IsSuccess() const { return ResponseCode == EConcertReplicationPutStateResponseCode::Success; }
};