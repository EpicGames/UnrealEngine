// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncSessionTypes.h"
#include "Muting.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/ReplicationStream.h"
#include "ReplicationActivity.generated.h"

#define UE_API CONCERTSYNCCORE_API

class FText;

/** Identifies the FConcertSyncReplicationEvent::Payload struct type */
UENUM()
enum class EConcertSyncReplicationActivityType : uint8
{
	None,
	
	/** Client left the replication session (either because they left the MU session or because they sent a FConcertReplication_LeaveEvent). */
	LeaveReplication,

	/** Client muted or unmuted some objects. */
	Mute,

	// ADD NEW ENTRIES ABOVE
	Count
};

/**
 * Contains the streams and authority a client had when they left a session.
 * Used for restoring content when the client rejoins.
 */
USTRUCT()
struct FConcertSyncReplicationPayload_LeaveReplication
{
	GENERATED_BODY()

	/** The streams the client had registered when they left. */
	UPROPERTY()
	TArray<FConcertReplicationStream> Streams;
	
	/** The objects the client had authority over when they left. */
	UPROPERTY()
	TArray<FConcertObjectInStreamID> OwnedObjects;

	friend bool operator==(const FConcertSyncReplicationPayload_LeaveReplication&, const FConcertSyncReplicationPayload_LeaveReplication&) = default;
	friend bool operator!=(const FConcertSyncReplicationPayload_LeaveReplication&, const FConcertSyncReplicationPayload_LeaveReplication&) = default;
};

USTRUCT()
struct FConcertSyncReplicationSummary_LeaveReplication
{
	GENERATED_BODY()
	
	FConcertSyncReplicationSummary_LeaveReplication() = default;
	explicit FConcertSyncReplicationSummary_LeaveReplication(const FConcertSyncReplicationPayload_LeaveReplication& Event)
		: OwnedObjects(Event.OwnedObjects)
	{}

	/** The objects the client had authority over when they left. */
	UPROPERTY()
	TArray<FConcertObjectInStreamID> OwnedObjects;
};

/** Stores objects that were muted / unmuted */
USTRUCT()
struct FConcertSyncReplicationPayload_Mute
{
	GENERATED_BODY()

	/** The request that changed mute state */
	UPROPERTY()
	FConcertReplication_ChangeMuteState_Request Request;
	
	friend bool operator==(const FConcertSyncReplicationPayload_Mute&, const FConcertSyncReplicationPayload_Mute&) = default;
	friend bool operator!=(const FConcertSyncReplicationPayload_Mute&, const FConcertSyncReplicationPayload_Mute&) = default;
};

/** Info displayed in the UI. */
USTRUCT()
struct FConcertSyncReplicationSummary_Mute
{
	GENERATED_BODY()

	FConcertSyncReplicationSummary_Mute() = default;
	explicit FConcertSyncReplicationSummary_Mute(const FConcertSyncReplicationPayload_Mute& Event)
		: Request(Event.Request)
	{}
	
	/** The request that changed mute state */
	UPROPERTY()
	FConcertReplication_ChangeMuteState_Request Request;
};

namespace UE::ConcertSyncCore
{
	inline FString GetReplicationActivityPayloadTypePathName(EConcertSyncReplicationActivityType Type)
	{
		static_assert(static_cast<int32>(EConcertSyncReplicationActivityType::Count) == 3, "If you added an EConcertSyncReplicationActivityType entry, update this switch");
		switch (Type)
		{
		case EConcertSyncReplicationActivityType::LeaveReplication: return FConcertSyncReplicationPayload_LeaveReplication::StaticStruct()->GetPathName();
		case EConcertSyncReplicationActivityType::Mute: return FConcertSyncReplicationPayload_Mute::StaticStruct()->GetPathName();
		default: ensureMsgf(false, TEXT("Unknown replication activity type (%u)"), static_cast<uint8>(Type)); return TEXT("");
		}
	}
}

/** Data for a replication event in a Concert Sync Session */
USTRUCT()
struct FConcertSyncReplicationEvent
{
	GENERATED_BODY()

	/** Identifies the Payload struct type */
	UPROPERTY()
	EConcertSyncReplicationActivityType ActivityType = EConcertSyncReplicationActivityType::None;

	/**
	 * A FConcertSyncReplicationPayload_X type depending on ActivityType.
	 * 
	 * Serialized into the database using Cbor; do not change (for simplicity we always assume it is Cbor).
	 * If you change the serialization method, you'll get a failing check.
	 * If for whatever reason you MUST save it using a different method, adjust FConcertSyncSessionDatabaseStatements::AddReplicationData
	 * and FConcertSyncSessionDatabaseStatements::SetReplicationData.
	 */
	UPROPERTY()
	FConcertSessionSerializedPayload Payload{ EConcertPayloadSerializationMethod::Cbor };

	FConcertSyncReplicationEvent() = default;
	template<typename TPayload>
	explicit FConcertSyncReplicationEvent(const TPayload& Data)
	{
		SetPayload(Data);
	}

	void SetPayload(const FConcertSyncReplicationPayload_LeaveReplication& Data)
	{
		ActivityType = EConcertSyncReplicationActivityType::LeaveReplication;
		Payload.SetTypedPayload(Data);
	}
	void SetPayload(const FConcertSyncReplicationPayload_Mute& Data)
	{
		ActivityType = EConcertSyncReplicationActivityType::Mute;
		Payload.SetTypedPayload(Data);
	}
	
	bool GetPayload(FConcertSyncReplicationPayload_LeaveReplication& Result) const
	{
		check(ActivityType == EConcertSyncReplicationActivityType::LeaveReplication);
		return Payload.GetTypedPayload(Result);
	}
	bool GetPayload(FConcertSyncReplicationPayload_Mute& Result) const
	{
		check(ActivityType == EConcertSyncReplicationActivityType::Mute);
		return Payload.GetTypedPayload(Result);
	}

	CONCERTSYNCCORE_API friend bool operator==(const FConcertSyncReplicationEvent& Left, const FConcertSyncReplicationEvent& Right);
	friend bool operator!=(const FConcertSyncReplicationEvent&, const FConcertSyncReplicationEvent&) = default;
};

/** Data for a replication activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncReplicationActivity : public FConcertSyncActivity
{
	GENERATED_BODY()

	FConcertSyncReplicationActivity()
	{
		EventType = EConcertSyncActivityEventType::Replication;
	}

	template<typename TPayload>
	explicit FConcertSyncReplicationActivity(const TPayload& PayloadData)
		: EventData(PayloadData)
	{
		EventType = EConcertSyncActivityEventType::Replication;
	}

	/** The replication event data associated with this activity */
	UPROPERTY()
	FConcertSyncReplicationEvent EventData;
};

/** Summary for a lock activity entry in a Concert Sync Session */
USTRUCT()
struct FConcertSyncReplicationActivitySummary : public FConcertSyncActivitySummary
{
	GENERATED_BODY()

	/** The type of replication event we summarize */
	UPROPERTY()
	EConcertSyncReplicationActivityType ActivityType { EConcertSyncReplicationActivityType::None };

	/** The summary data. The underlying type depends on ActivityType. */
	UPROPERTY()
	FConcertSessionSerializedPayload Payload{ EConcertPayloadSerializationMethod::Cbor };

	bool GetSummaryData(FConcertSyncReplicationSummary_LeaveReplication& Data) const
	{
		check(ActivityType == EConcertSyncReplicationActivityType::LeaveReplication);
		return Payload.GetTypedPayload(Data);
	}
	bool GetSummaryData(FConcertSyncReplicationSummary_Mute& Data) const
	{
		check(ActivityType == EConcertSyncReplicationActivityType::Mute);
		return Payload.GetTypedPayload(Data);
	}

	/** Create this summary from a replication event */
	static UE_API FConcertSyncReplicationActivitySummary CreateSummaryForEvent(const FConcertSyncReplicationEvent& InEvent);

	/** Gets the title for this summary */
	UE_API FText ToDisplayTitle() const;

protected:
	
	//~ Begin FConcertSyncActivitySummary Interface
	UE_API virtual FText CreateDisplayText(const bool InUseRichText) const override;
	UE_API virtual FText CreateDisplayTextForUser(const FText InUserDisplayName, const bool InUseRichText) const override;
	//~ End FConcertSyncActivitySummary Interface
};

#undef UE_API
