// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Replication/Data/ClientQueriedInfo.h"
#include "Replication/Messages/SyncControl.h"
#include "RestoreContent.generated.h"

UENUM(Flags) 
enum class EConcertReplicationRestoreContentFlags : uint8
{
	None = 0,

	/**
	 * If set, the client's final streams & authority shall be the union of the current registered streams & authority and that of the activity.
	 * If unset, the client's streams and authority are cleared (effectively replaced by what is stored in the activity).
	 */
	RestoreOnTop = 1 << 0,

	/** If set, checks that there is no other client with the same display and device name. If there is, the request fails with NameConflict. */
	ValidateUniqueClient = 1 << 1,

	/** Also restore the authority the client had.  */
	RestoreAuthority = 1 << 2,

	/** The response should include the new stream and authority state. */
	SendNewState = 1 << 3,

	/** Also restore the mute states of the objects. This is done by going through the activity history and replaying the mute & unmute actions. */
	RestoreMute = 1 << 4,

	StreamsAndAuthority = RestoreAuthority,
	All = StreamsAndAuthority | RestoreMute
};
ENUM_CLASS_FLAGS(EConcertReplicationRestoreContentFlags);

UENUM()
enum class EConcertReplicationAuthorityRestoreMode : uint8
{
	// Suppose client 1 has registered (SceneComponent, { RelativeLocation }) and has authority over SceneComponent.
	// If client 2 tries to restore (SceneComponent, { ComponentTags }), it will work in each of the below cases (client 1 does not own ComponentTags).
	// If client 2 tries to restore (SceneComponent, { ComponentTags }), it will only work for IncludeAlreadyOwnedObjectPropertiesInStream.

	// The following fields apply even when EConcertReplicationRestoreContentFlags::RestoreAuthority is not set.
	
	/**
	 * If another client already has authority over a would-be restored object' restored properties, do not restore object in the client's stream.
	 * This only affects the restore properties; if RestoreOnTop is set and there are conflicts with non-restored properties, this filter does not apply.
	 */
	ExcludeAlreadyOwnedObjectPropertiesFromStream,
	/** If another client already has authority over a would-be restored object's restored properties, restore it into the client's stream (but obviously don't take authority). */
	IncludeAlreadyOwnedObjectPropertiesInStream,
	/** If another client already has authority over a would-be restored object's restored properties, do not restore anything. */
	AllOrNothing
};

/**
 * Request the sending client's stream and optionally authority to be aggregated with what a client had when they left the session.
 *
 * A FConcertSyncFConcertSyncReplicationActivity is generated whenever a client leaves a replication session which has
 * EConcertSyncSessionFlags::ShouldEnableReplicationActivities flag set. This activity contains the stream content and authority of the client at that
 * point. This request is used to restore the client's state when they last left.
 */
USTRUCT()
struct FConcertReplication_RestoreContent_Request
{
	GENERATED_BODY()

	/** Describes what and how content is to be restored. */
	UPROPERTY()
	EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority;

	/** If EConcertReplicationRestoreContentFlags::RestoreAuthority is set, describes how to deal with authority conflicts. */
	UPROPERTY()
	EConcertReplicationAuthorityRestoreMode AuthorityRestorationMode = EConcertReplicationAuthorityRestoreMode::ExcludeAlreadyOwnedObjectPropertiesFromStream;
	
	/**
	 * The ID of an activity that contains a client's replication state.
	 * The activity must be a FConcertSyncReplicationActivity with whose EventData.ActivityType == EConcertSyncReplicationActivityType::LeaveReplication.
	 * 
	 * If left unset, use the latest activity of matching the sending endpoint's display and device name.
	 * This can be a different client.
	 */
	UPROPERTY()
	TOptional<int64> ActivityId;
};

UENUM()
enum class EConcertReplicationRestoreErrorCode : uint8
{
	/** The request timed out. */
	Timeout,

	/**
	 * The request was successful.
	 * The request is also successful if there was no activity to restore since the client is in the "right" state.
	 */
	Success,

	/** Requesting client has not joined replication. */
	Invalid,
	/** EConcertSyncSessionFlags::ShouldEnableReplicationActivities is not set. */
	NotSupported,
	/** The request's ActivityId was set but did not point to an appropriate activity. */
	NoSuchActivity,
	/** Nothing could be restored because another client with the same display and device name already is in the session. */
	NameConflict,
	/** EConcertReplicationAuthorityRestoreMode::AllOrNothing was set and another client had authority over one of the would-be restore objects. */
	AuthorityConflict
};

namespace UE::ConcertSyncCore
{
	inline FString LexToString(EConcertReplicationRestoreErrorCode ErrorCode)
	{
		switch (ErrorCode)
		{
		case EConcertReplicationRestoreErrorCode::Timeout: return TEXT("Timeout");
		case EConcertReplicationRestoreErrorCode::Success: return TEXT("Success");
		case EConcertReplicationRestoreErrorCode::Invalid: return TEXT("Invalid");
		case EConcertReplicationRestoreErrorCode::NotSupported: return TEXT("NotSupported");
		case EConcertReplicationRestoreErrorCode::NoSuchActivity: return TEXT("NotSupported");
		case EConcertReplicationRestoreErrorCode::NameConflict: return TEXT("NameConflict");
		case EConcertReplicationRestoreErrorCode::AuthorityConflict: return TEXT("AuthorityConflict");
			default: checkNoEntry(); return TEXT("Unknown");
		}
	}
}

USTRUCT()
struct FConcertReplication_RestoreContent_Response
{
	GENERATED_BODY()

	UPROPERTY()
	EConcertReplicationRestoreErrorCode ErrorCode = EConcertReplicationRestoreErrorCode::Timeout;

	/**
	 * This includes the full stream and authority content if ErrorCode == EConcertReplicationRestoreErrorCode::Success and the request had
	 * EConcertReplicationRestoreContentFlags::SendNewState set.
	 */
	UPROPERTY()
	FConcertQueriedClientInfo ClientInfo;
	
	/**
	 * The full sync control the client has on the server.
	 * Only valid if ErrorCode == EConcertReplicationRestoreErrorCode::Success.
	 * The client does not receive a separate event for this and is expected to apply it instantly.
	 */
	UPROPERTY()
	FConcertReplication_ChangeSyncControl SyncControl;

	bool IsSuccess() const { return ErrorCode == EConcertReplicationRestoreErrorCode::Success; }
};