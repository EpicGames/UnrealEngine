// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/AnyOf.h"
#include "Replication/Data/ObjectIds.h"
#include "SyncControl.generated.h"

/**
 * Notifies clients about the ability to start or stop replicating specific objects.
 *
 * Sync control serves to inform clients whether the data they wish to replicate has any receivers.
 * Replicating data when no one is listening is unnecessary.
 *
 * Until a sync control event is received, clients should assume no interest in the data.
 *
 * Special Circumstances:
 * 1. FConcertReplication_Join_Request: The server doesn't send any FConcertReplication_ChangeSyncControl event in response to the handshake.
 *    Instead, the resulting sync control is in FConcertReplication_Join_Response::SyncControl.
 * 2. Explicit authority change via FConcertReplication_ChangeAuthority_Request: No sync control event is sent.
 *	  FConcertReplication_ChangeAuthority_Response::SyncControl contains the objects for which the client received instant sync control.
 *    Objects losing sync control are implicitly determined by using FSyncControlState::Aggregate.
 * 3. Implicit authority change via FConcertReplication_ChangeStream_Request: No sync control event is generated as its result is implicit from the request and response.
 *	  Stream changes can only cause the client to lose authority, thus sync control, over objects.
 *    Therefore, objects losing sync control can be enumerated using FSyncControlState::Aggregate.
 *
 * Example Flow:
 * 1. Client 1 registers stream 's' with object 'x' (and associated properties) using FConcertReplication_ChangeStream_Request.
 * 2. Client 1 takes authority over 'x' with 's' using FConcertReplication_ChangeAuthority_Request.
 * 3. Client 2 joins (and wants to receive the data).
 * 4. Server sends FConcertReplication_ChangeSyncControl to client 1 to begin replicating 'x' in 's'.
 * 5. Client 2 leaves.
 * 6. Server sends FConcertReplication_ChangeSyncControl to client 1 to cease replicating 'x' in 's'.
 */
USTRUCT()
struct FConcertReplication_ChangeSyncControl
{
	GENERATED_BODY()

	/** Toggles the objects (which the receiving client 1. has previously registered and 2. taken authority over) to replicate or not. */
	UPROPERTY()
	TMap<FConcertObjectInStreamID, bool> NewControlStates;

	bool IsEmpty() const { return NewControlStates.IsEmpty(); }
	
	bool DoesAtLeastOneObjectGainSyncControl() const { return Algo::AnyOf(NewControlStates, [](const TPair<FConcertObjectInStreamID, bool>& Pair){ return Pair.Value; }); }
	bool DoesAtLeastOneObjectLoseSyncControl() const { return Algo::AnyOf(NewControlStates, [](const TPair<FConcertObjectInStreamID, bool>& Pair){ return !Pair.Value; }); }
};