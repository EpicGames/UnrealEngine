// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorLabelRemappingInput.h"
#include "Misc/ObjectUtils.h"
#include "Replication/Data/ActorLabelRemapping.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "Private/RemapAlgorithm.h"

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Templates/FunctionFwd.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

enum class EBreakBehavior : uint8;

namespace UE::ConcertSyncCore
{
	/**
	 * Creates the remapping data for a replication map.
	 * @param Origin The replication map to generate the remapping data for
	 * @param GetLabelFunc Gets the label of an object
	 * @param GetClassFunc Gets the class of an object
	 * @return The remapping data you can use to remap Origin to.
	 */
	template<CGetObjectLabelCallable TGetObjectLabel, CGetObjectClassCallable TGetObjectClass>
	void GenerateRemappingData(
		const FConcertObjectReplicationMap& Origin,
		TGetObjectLabel&& GetLabelFunc,
		TGetObjectClass&& GetClassFunc,
		FConcertReplicationRemappingData& Result
		);
	/** Util version that creates a new replication mapping. */
	template<CGetObjectLabelCallable TGetObjectLabel, CGetObjectClassCallable TGetObjectClass>
	FConcertReplicationRemappingData GenerateRemappingData(
		const FConcertObjectReplicationMap& Origin,
		TGetObjectLabel&& GetLabelFunc,
		TGetObjectClass&& GetClassFunc
		)
	{
		FConcertReplicationRemappingData Result;
		GenerateRemappingData(Origin, GetLabelFunc, GetClassFunc, Result);
		return Result;
	}

	/**
	 * Attempts to remap all objects in Origin according to the labels saved in RemappingData.
	 * Calls ProcessRemapping for each object so remapped.
	 *
	 * @see FConcertReplicationRemappingData for an example.
	 *
	 * @param Origin The replicationmap to remap
	 * @param RemappingData The data based on which to remap
	 * @param IsRemappingCompatibleFunc Decides whether this remapping is compatible
	 * @param ForEachObjectWithLabelFunc Lists all objects with a particular label.
	 * @param GetLabelFunc Gets the label of an object 
	 * @param ProcessRemapping Called for every remapped object
	 */
	template
	<
		CIsRemappingCompatibleCallable TIsRemappingCompatible,
		CForEachObjectWithLabelCallable TForEachObjectWithLabel,
		CGetObjectLabelCallable TGetObjectLabel,
		CProcessRemappingCallbable TProcessRemapping
	>
	void RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		TIsRemappingCompatible&& IsRemappingCompatibleFunc,
		TForEachObjectWithLabel&& ForEachObjectWithLabelFunc,
		TGetObjectLabel&& GetLabelFunc,
		TProcessRemapping&& ProcessRemapping
		)
	{
		Private::TRemapAlgorithm(Origin, RemappingData, IsRemappingCompatibleFunc, ForEachObjectWithLabelFunc, GetLabelFunc)
			.Run([&ProcessRemapping](const FSoftObjectPath& Original, const FSoftObjectPath& Target)
			{
				ProcessRemapping(Original, Target);
			});
	}
	/** Alternate version that directly writes into OutTargetMap. */
	template<CIsRemappingCompatibleCallable TIsRemappingCompatible, CForEachObjectWithLabelCallable TForEachObjectWithLabel, CGetObjectLabelCallable TGetObjectLabel>
	void RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		TIsRemappingCompatible&& IsRemappingCompatibleFunc,
		TForEachObjectWithLabel&& ForEachObjectWithLabelFunc,
		TGetObjectLabel&& GetLabelFunc,
		FConcertObjectReplicationMap& OutTargetMap
		)
	{
		Private::TRemapAlgorithm(Origin, RemappingData, IsRemappingCompatibleFunc, ForEachObjectWithLabelFunc, GetLabelFunc)
			.Run([&Origin, &OutTargetMap](const FSoftObjectPath& Original, const FSoftObjectPath& Target)
			{
				OutTargetMap.ReplicatedObjects.Add(Target, Origin.ReplicatedObjects[Original]);
			});
	}
	/** Util version that creates a new replication mapping. */
	template<CIsRemappingCompatibleCallable TIsRemappingCompatible, CForEachObjectWithLabelCallable TForEachObjectWithLabel, CGetObjectLabelCallable TGetObjectLabel>
	FConcertObjectReplicationMap RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		TIsRemappingCompatible&& IsRemappingCompatibleFunc,
		TForEachObjectWithLabel&& ForEachObjectWithLabelFunc,
		TGetObjectLabel&& GetLabelFunc
		)
	{
		FConcertObjectReplicationMap Result;
		RemapReplicationMap(Origin, RemappingData, IsRemappingCompatibleFunc, ForEachObjectWithLabelFunc, GetLabelFunc, Result);
		return Result;
	}
}

namespace UE::ConcertSyncCore
{
	template <CGetObjectLabelCallable TGetObjectLabel, CGetObjectClassCallable TGetObjectClass>
	void GenerateRemappingData(
		const FConcertObjectReplicationMap& Origin,
		TGetObjectLabel&& GetLabelFunc,
		TGetObjectClass&& GetClassFunc,
		FConcertReplicationRemappingData& Result
		)
	{
		for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : Origin.ReplicatedObjects)
		{
			const FSoftObjectPath& ObjectPath = Pair.Key;
			const TOptional<FSoftObjectPath> OwningActorPath = GetActorPathIn(ObjectPath);
			if (!OwningActorPath || Result.ActorData.Contains(*OwningActorPath))
			{
				continue;
			}

			// GetLabelFunc and GetClassFunc *could* be combined into a single func to reduce ObjectPtr.Resolve count from 2 to 1 but the overhead
			// should be minimal for the 2nd resolve because ObjectPtr will cache the value; we're making it easier for the API user by having 2 funcs.
			const FSoftObjectPtr OwningActorPtr{ *OwningActorPath };
			const TOptional<FString> Label = GetLabelFunc(OwningActorPtr);
			const FSoftClassPath Class = GetClassFunc(OwningActorPtr);
			if (Label && !Class.IsNull())
			{
				FConcertReplicationRemappingData_Actor& ActorData = Result.ActorData.Add(*OwningActorPath);
				ActorData.Label = *Label;
				ActorData.Class = Class;
			}
		}
	}
}
