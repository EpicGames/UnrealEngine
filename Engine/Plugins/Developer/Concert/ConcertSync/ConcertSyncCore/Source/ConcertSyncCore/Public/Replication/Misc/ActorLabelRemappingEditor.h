// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ActorLabelRemappingInput.h"
#include "ActorLabelRemappingCore.h"
#include "Private/EditorRemappingUtils.h"

class UWorld;

namespace UE::ConcertSyncCore
{
	// The functions in ActorLabelRemappingCore.h are generic and operate only on FSoftObjectPaths.
	// The following functions are only useful in builds that have AActor instances.
	
	/** This version of GenerateRemappingData uses AActor::ActorLabel. */
	void GenerateRemappingData(
		const FConcertObjectReplicationMap& Origin,
		FConcertReplicationRemappingData& Result
		);
	/** Util version that creates a new replication mapping. */
	FConcertReplicationRemappingData GenerateRemappingData(const FConcertObjectReplicationMap& Origin);

	/** This version of RemapReplicationMap uses AActor::ActorLabel and tries to remap to actors in TargetWorld. */
	template<CProcessRemappingCallbable TProcessRemapping>
	void RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		const UWorld& TargetWorld,
		TProcessRemapping&& ProcessRemapping
		);
	/** Alternate version that directly writes into OutTargetMap. */
	void RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		const UWorld& TargetWorld,
		FConcertObjectReplicationMap& OutTargetMap
		);
	/** Util version that creates a new replication mapping. */
	FConcertObjectReplicationMap RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		const UWorld& TargetWorld
		);
}

namespace UE::ConcertSyncCore // Inline defs
{
	FORCEINLINE void GenerateRemappingData(
		const FConcertObjectReplicationMap& Origin,
		FConcertReplicationRemappingData& Result
		)
	{
		GenerateRemappingData(
			Origin,
			[](const FSoftObjectPtr& Object){ return Private::GetActorLabel(Object); },
			[](const FSoftObjectPtr& Object) { return Private::GetClassPath(Object); },
			Result
			);
	}
	
	FORCEINLINE FConcertReplicationRemappingData GenerateRemappingData(const FConcertObjectReplicationMap& Origin)
	{
		FConcertReplicationRemappingData Result;
		GenerateRemappingData(Origin, Result);
		return Result;
	}
	
	template <CProcessRemappingCallbable TProcessRemapping>
	void RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		const UWorld& TargetWorld,
		TProcessRemapping&& ProcessRemapping)
	{
		Private::GenericRemapReplicationMap(Origin, RemappingData, TargetWorld, ProcessRemapping);
	}
	
	FORCEINLINE void RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		const UWorld& TargetWorld,
		FConcertObjectReplicationMap& OutTargetMap
		)
	{
		Private::GenericRemapReplicationMap(Origin, RemappingData, TargetWorld, OutTargetMap);
	}
	
	FORCEINLINE FConcertObjectReplicationMap RemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		const UWorld& TargetWorld
		)
	{
		FConcertObjectReplicationMap Result;
		RemapReplicationMap(Origin, RemappingData, TargetWorld, Result);
		return Result;
	}
}
#endif