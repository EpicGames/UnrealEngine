// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/FunctionFwd.h"

#include <type_traits>

class FString;

enum class EBreakBehavior : uint8;

struct FSoftClassPath;
struct FSoftObjectPath;
struct FSoftObjectPtr;
template<typename OptionalType> struct TOptional;

namespace UE::ConcertSyncCore
{
	/**
	 * Lets you decide whether OriginObject can be remapped to PossibleTarget.
	 *
	 * This should check the following:
	 * - does PossibleTarget exist? PossibleTarget is the result of a simple string replacement in the object path.
	 * - is PossibleTarget's class compatible with OriginClass? You can decide what compatible means - you may want exact classes or subclasses.
	 *
	 * @param OriginObject The original object that is being evaluated as remapping candidate.
	 *	Important: if OriginObject points to an actor, then the original replication map is not guaranted to contain OriginObject.
	 *	This happens e.g. if the original map contained "/Game/World.World:PersistentLevel.Actor.Component" but not its owning actor.
	 *	In that case, this callback is being asked, whether the subobjects ("Component" in this case) can be remapped to actor OriginObject points at.
	 * @param OriginClass The class the OriginObject is expected to be compatible with. 
	 * @param PossibleTargetActor The owning actor of PossibleTarget. If PossibleTarget is an actor, then PossibleTarget == PossibleTargetActor.
	 *	This arg may be useful because points to the actual UObject instance (if in engine build).
	 * @param PossibleTarget The object that OriginObject will be remapped to if this function returns true.
	 */
	template<typename TCallback>
	concept CIsRemappingCompatibleCallable = std::is_invocable_r_v<bool, TCallback,
		const FSoftObjectPath& /*OriginObject*/,
		const FSoftClassPath& /*OriginClass*/,
		const FSoftObjectPtr& /*PossibleTargetActor*/,
		const FSoftObjectPath& /*PossibleTarget*/
	>;

	/** For each object that has the given Label invokes Consumer until either Consumer returns EBreakBehavior::Break or there are no more */
	template<typename TCallback>
	concept CForEachObjectWithLabelCallable = std::is_invocable_r_v<void, TCallback,
		const FString& /*Label*/,
		TFunctionRef<EBreakBehavior(const FSoftObjectPtr& Actor)> /*Consumer*/
		>;
	
	/**
	 * Gets the label of an actor.
	 * 
	 * Return unset if any:
	 * - Object not available
	 * - you don't want the label in the result.
	 */
	template<typename TCallback>
	concept CGetObjectLabelCallable = std::is_invocable_r_v<TOptional<FString>, TCallback, const FSoftObjectPtr& /*Object*/>;
	/**
	 * Gets an object's class.
	 * Return an empty class path if Object cannot be resolved.
	 */
	template<typename TCallback>
	concept CGetObjectClassCallable = std::is_invocable_r_v<FSoftClassPath, TCallback, const FSoftObjectPtr& /*Object*/>;
	/** Process a remapped object */
	template<typename TCallback>
	concept CProcessRemappingCallbable = std::is_invocable_r_v<void, TCallback, const FSoftObjectPath& /*OriginalObject*/, const FSoftObjectPath& /*RemappedObject*/> ;
}