// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EBreakBehavior.h"
#include "Templates/Function.h"

struct FConcertObjectReplicationMap;
struct FSoftObjectPath;

namespace UE::ConcertSyncCore::Replication
{
	/** Defines a way to obtain streaming information about clients. */
	class IReplicationGroundTruth
	{
	public:

		/** Provides a way to extract all streams registered to a given client. */
		virtual void ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)> Callback) const = 0;

		/** Iterates through all clients have registered to send any data. */
		virtual void ForEachClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const = 0;

		/** @return Whether ClientId's stream StreamId has authority over ObjectPath. */
		virtual bool HasAuthority(const FGuid& ClientId, const FGuid& StreamId, const FSoftObjectPath& ObjectPath) const = 0; 

		virtual ~IReplicationGroundTruth() = default;
	};
}
