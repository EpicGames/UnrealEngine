// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Replication/Misc/ActorLabelRemappingCore.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

namespace UE::ConcertSyncCore::Private
{
	constexpr bool bCreateActorLabelDuringRead = false;
		
	inline TOptional<FString> GetActorLabel(const FSoftObjectPtr& Object)
	{
		const UObject* ResolvedObject = Object.Get();
		const AActor* Actor = Cast<AActor>(ResolvedObject);
		const FString Label = Actor ? Actor->GetActorLabel(bCreateActorLabelDuringRead) : FString{};
		return Label.IsEmpty() ? TOptional<FString>{} : Label;
	}

	inline FSoftClassPath GetClassPath(const FSoftObjectPtr& Object)
	{
		const UObject* ResolvedObject = Object.Get();
		return ResolvedObject ? ResolvedObject->GetClass() : FSoftClassPath{};
	}
	
	inline TMap<FString, TArray<FSoftObjectPtr>> CacheByActorLabel(const UWorld& World)
	{
		TMap<FString, TArray<FSoftObjectPtr>> LabelsToInfo;
		for (TActorIterator<AActor> ActorIt(&World); ActorIt; ++ActorIt)
		{
			const FString Label = ActorIt->GetActorLabel(bCreateActorLabelDuringRead);
			if (Label.IsEmpty())
			{
				continue;;
			}
			
			LabelsToInfo.FindOrAdd(Label).Emplace(*ActorIt);
		}
		return LabelsToInfo;
	}

	/** This functions sole purpose is to avoid code duplication for the overloads of GenerateRemappingData. */
	template<typename TFinalArg>
	static void GenericRemapReplicationMap(
		const FConcertObjectReplicationMap& Origin,
		const FConcertReplicationRemappingData& RemappingData,
		const UWorld& TargetWorld,
		TFinalArg&& Argument
	)
	{
		const auto IsRemappingCompatibleFunc = [](
			const FSoftObjectPath&,
			const FSoftClassPath& OriginClass,
			const FSoftObjectPtr&,
			const FSoftObjectPath& PossibleTarget
			)
		{
			UObject* Object = PossibleTarget.ResolveObject();
			if (!Object)
			{
				return false;
			}
			
			const bool bAreClassesSame = FSoftClassPath(Object->GetClass()) == OriginClass;
			return bAreClassesSame;
		};

		using namespace Private;
		const TMap<FString, TArray<FSoftObjectPtr>> LabelCache = CacheByActorLabel(TargetWorld);
		const auto ForEachObjectWithLabel = [&LabelCache](
			const FString& Label,
			TFunctionRef<EBreakBehavior(const FSoftObjectPtr& Actor)> Consumer
			)
		{
			const TArray<FSoftObjectPtr>* ActorsWithlabel = LabelCache.Find(Label);
			if (!ActorsWithlabel)
			{
				return;
			}

			for (const FSoftObjectPtr& Actor : *ActorsWithlabel)
			{
				if (Consumer(Actor) == EBreakBehavior::Break)
				{
					break;
				}
			}
		};
	
		RemapReplicationMap(
			Origin,
			RemappingData,
			IsRemappingCompatibleFunc,
			ForEachObjectWithLabel,
			[](const FSoftObjectPtr& Object){ return GetActorLabel(Object); },
			Argument
			);
	}
}
#endif