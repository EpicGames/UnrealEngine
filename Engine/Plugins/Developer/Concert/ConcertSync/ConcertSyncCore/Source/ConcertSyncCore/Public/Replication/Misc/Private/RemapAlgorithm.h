// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertLogGlobal.h"
#include "Misc/ObjectPathHierarchy.h"
#include "Misc/ObjectUtils.h"
#include "Replication/Data/ActorLabelRemapping.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "Replication/Misc/ActorLabelRemappingInput.h"

#include <type_traits>

namespace UE::ConcertSyncCore::Private
{
	/**
	 * This algorithm remaps FSoftObjectPaths from an origin FConcertObjectReplicationMap to target FSoftObjectPaths based on the owning actors
	 * sharing actor labels.
	 * @see FConcertReplicationRemappingData for example.
	 */
	template<CIsRemappingCompatibleCallable TIsRemappingCompatible, CForEachObjectWithLabelCallable TForEachObjectWithLabel, CGetObjectLabelCallable TGetObjectLabel>
	class TRemapAlgorithm
	{
	public:

		TRemapAlgorithm(
			const FConcertObjectReplicationMap& Origin,
			const FConcertReplicationRemappingData& RemappingData,
			const TIsRemappingCompatible& ForEachObjectWithOuterFunc,
			const TForEachObjectWithLabel& ForEachObjectWithLabelFunc,
			const TGetObjectLabel& GetLabelFunc
			)
			: Origin(Origin)
			, RemappingData(RemappingData)
			, IsRemappingCompatibleFunc(ForEachObjectWithOuterFunc)
			, ForEachObjectWithLabelFunc(ForEachObjectWithLabelFunc)
			, GetLabelFunc(GetLabelFunc)
		{}
		
		template<CProcessRemappingCallbable TProcessRemappingLambda>
		void Run(TProcessRemappingLambda&& ProcessRemapping)
		{
			// 1. Build object hierarchy of Origin & find all actors that we are supposed to remap (those that have a label)
			BuildOriginObjectHierarchyAndActorsNeedingRemapping();
			
			// 2. For each re-mappable actor, build the list of target candidates using the hierarchy to check the requirements.
			ComputeRemappingCandidates();
			
			// 3. In order of least-remaining target candidates, remap actors; uses the hierarchy to remap actor subobjects.
			RemapActors(ProcessRemapping);
		}

	private:
	
		// Input data to the algorithm
		const FConcertObjectReplicationMap& Origin;
		const FConcertReplicationRemappingData& RemappingData;
		const TIsRemappingCompatible& IsRemappingCompatibleFunc;
		const TForEachObjectWithLabel& ForEachObjectWithLabelFunc;
		const TGetObjectLabel& GetLabelFunc;
		
		/****************************** 1. Discover actors with labels ******************************/
		
		struct FActorAndClassPair
		{
			/** The path of an actor that is directly or indirectly referenced by the replication map. */
			const FSoftObjectPath OriginActor;
			/** The class that OriginActor is expected to have, as recorded by FConcertReplicationRemappingData_Actor. */
			const FSoftClassPath& Class;

			FActorAndClassPair(FSoftObjectPath OriginActor, const FSoftClassPath& Class)
				: OriginActor(MoveTemp(OriginActor))
				, Class(Class)
			{}
		};
		
		/** Hierarchy that is used by ComputeTargetCandidates to efficiently determine objects that need to be reassigned. */
		FObjectPathHierarchy ObjectPathHierarchy;
		/** Maps actor labels to object path from the original map that had that label. */
		TMap<FString, TArray<FActorAndClassPair>> LabelsToPendingActors;
		int32 ExpectedNumberRemappedActors = 0;

		/**
		 * Processes the input replication map:
		 * - Builds ObjectPathHierarchy based on what the original replication map contained - this is used in later for analyzing the hierarchy.
		 * - Groups all actors that need processing by label in LabelsToPendingActors.
		 */
		void BuildOriginObjectHierarchyAndActorsNeedingRemapping()
		{
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : Origin.ReplicatedObjects)
			{
				const FSoftObjectPath& ObjectPath = Pair.Key;
				ObjectPathHierarchy.AddObject(ObjectPath);

				// It could be that ObjectPath is a component and the owning actor is not replicated / was not looped, yet.
				// Handle that case here.
				const TOptional<FSoftObjectPath> OwningActorPath = GetActorPathIn(ObjectPath);
				const FConcertReplicationRemappingData_Actor* ActorData = OwningActorPath ? RemappingData.ActorData.Find(*OwningActorPath) : nullptr;
				if (!ActorData)
				{
					continue;
				}
				
				TArray<FActorAndClassPair>& Actors = LabelsToPendingActors.FindOrAdd(ActorData->Label);
				const bool bAlreadyContainsActor = Actors.ContainsByPredicate([&OwningActorPath](const FActorAndClassPair& Pair)
				{
					return Pair.OriginActor == *OwningActorPath;
				});
				// Since we do GetActorPathIn for every subobject above, we may already have added the actor
				if (!bAlreadyContainsActor)
				{
					++ExpectedNumberRemappedActors;
					Actors.Emplace(*OwningActorPath, ActorData->Class);
				}
			}
		}
		
		/****************************** 2. Find candidates that actors can be remapped to ******************************/
		
		/**
		 * The max number of actors in a level we expect to have share the same label.
		 * We expect a typical level to have at most 4 actors with the same label... if we're wrong we're punished with dynamic memory allocs.
		 */
		enum { ExpectedNumLabelCollisions = 4 };
		
		template<typename T>
		using TSmallInlineArray = TArray<T, TInlineAllocator<ExpectedNumLabelCollisions>>;
		
		struct FActorWithSolutions
		{
			/**
			 * An actor directly or indirectly referenced by the Origin replication map.
			 * This is a reference to FActorAndClassPair::OriginActor.
			 */
			const FSoftObjectPath& OriginActor;
			/** Actors that OriginActor can be replaced with. It had been validated that the hierarchy is compatible. */
			TSmallInlineArray<FSoftObjectPath> PossibleSolutions;

			explicit FActorWithSolutions(const FSoftObjectPath& OriginActor)
				: OriginActor(OriginActor)
			{}
		};
		/** Holds original paths that can be remapped. This is a TArray instead of TMap because RemapActors will iterate it often. */
		TArray<FActorWithSolutions> ObjectsWithSolutions;

		/**
		 * Goes through all actors of which the hierarchy needs remapping.
		 * Invokes ForEachObjectWithLabelFunc to get candidate actors with the same label.
		 * Proceeds to check whether the candidates' subobject hierarchy is compatible and if so, adds the candiate to ObjectsWithSolutions.
		 */
		void ComputeRemappingCandidates()
		{
			if (ExpectedNumberRemappedActors == 0)
			{
				return;
			}
			
			ObjectsWithSolutions.Reserve(ExpectedNumberRemappedActors);
			for (TPair<FString, TArray<FActorAndClassPair>>& Pair : LabelsToPendingActors)
			{
				// If we find no potential solutions, this will keep some entries with PossibleSolutions.Num() == 0. RemapActors handles that.
				TArray<FActorAndClassPair>& PendingActors = Pair.Value;
				const int32 StartIndex = ObjectsWithSolutions.Num();
				Algo::Transform(PendingActors, ObjectsWithSolutions, [](const FActorAndClassPair& Pending)
				{
					return FActorWithSolutions{ Pending.OriginActor };
				});
				const auto GetSolutionData = [this, StartIndex](int32 ActorIndex) -> FActorWithSolutions&
				{
					const int32 SolutionIdx = StartIndex + ActorIndex;
					return ObjectsWithSolutions[SolutionIdx];
				};
				
				ForEachObjectWithLabelFunc(Pair.Key, [this, &PendingActors, &GetSolutionData](const FSoftObjectPtr& TargetCandidate)
				{
					FindSolutionsIn(PendingActors, TargetCandidate, GetSolutionData);
					// In theory, no further processing is needed once every remapped actor has PendingActors.Num() solutions in PossibleSolutions.
					// In practice, we don't expect that many actors with the label in the level to warrant the implementation.
					return EBreakBehavior::Continue;
				});
			}
		}

		/** Analyzes for every provided actor whether PossibleTargetActor is a suitable substitute. */
		template<typename TGetActorSolutionData>
		requires std::is_invocable_r_v<FActorWithSolutions&, TGetActorSolutionData, int32>
		void FindSolutionsIn(
			const TArray<FActorAndClassPair>& PendingActors,
			const FSoftObjectPtr& TargetCandidate,
			TGetActorSolutionData&& GetSolutionData
			)
		{
			for (int32 ActorIdx = 0; ActorIdx < PendingActors.Num(); ++ActorIdx)
			{
				const FActorAndClassPair& PendingActor = PendingActors[ActorIdx];
				const FSoftObjectPath& OriginalActor = PendingActor.OriginActor;
				const bool bIsActorObjectCompatible = IsRemappingCompatibleFunc(
					OriginalActor, PendingActor.Class, TargetCandidate, TargetCandidate.ToSoftObjectPath()
					);
				if (bIsActorObjectCompatible && IsHierarchyCompatible(OriginalActor, TargetCandidate))
				{
					FActorWithSolutions& SolutionData = GetSolutionData(ActorIdx);
					SolutionData.PossibleSolutions.Add(TargetCandidate.ToSoftObjectPath());
				}
			}
		}

		/** Validates that PossibleTargetActor's replicated hierarchy is compatible with that of OriginalActor. */
		bool IsHierarchyCompatible(const FSoftObjectPath& OriginalActor, const FSoftObjectPtr& TargetCandidate) const
		{
			bool bIsHierarchyCompatible = true;
			
			ObjectPathHierarchy.TraverseTopToBottom([this, &TargetCandidate, &bIsHierarchyCompatible](const FChildRelation& Relation)
			{
				// E.g. hierarchy "Actor", "Actor.Foo", "Actor.Foo.Bar" where replication only maps "Actor" and "Actor.Foo.Bar".
				// Then "Actor" and "Actor.Foo.Bar" are explicit, and "Actor.Foo" is implicit (needing no validation).
				const bool bIsReplicated = Relation.Child.Type == EHierarchyObjectType::Explicit; 
				if (!bIsReplicated)
				{
					return ETreeTraversalBehavior::Continue;
				}

				const FSoftObjectPath& OriginalObject = Relation.Child.Object;
				const TOptional<FSoftObjectPath> TargetPath = ReplaceActorInPath(OriginalObject, TargetCandidate.ToSoftObjectPath());
				if (!ensure(TargetPath.IsSet()))
				{
					// We don't expect this case to occur - so log it.
					UE_LOGF(LogConcert, Warning,
						"Remapping: OriginalObject %ls is no world object, or PossibleTargetActor %ls is not an actor",
						*OriginalObject.ToString(),
						*TargetCandidate.ToString()
						);
					return ETreeTraversalBehavior::Continue;
				}
				const FConcertReplicatedObjectInfo& ReplicationInfo = Origin.ReplicatedObjects[OriginalObject];
				bIsHierarchyCompatible &= IsRemappingCompatibleFunc(OriginalObject, ReplicationInfo.ClassPath, TargetCandidate, *TargetPath);
						
				return bIsHierarchyCompatible ? ETreeTraversalBehavior::Continue : ETreeTraversalBehavior::Break;
			}, OriginalActor);

			return bIsHierarchyCompatible;
		}
		
		/****************************** 3. Choose a solution and remap actor hierarchy to it ******************************/

		/**
		 * For each object that needs remapping, tries to pick a candidate we previously determined suitable.
		 * 
		 * The remapping is done greedily by always remapping the actor with least remaining options first with the goal of preventing accidental
		 * starvation, see more explaination below.
		 */
		template<CProcessRemappingCallbable TProcessRemappingLambda>
		void RemapActors(TProcessRemappingLambda&& ProcessRemapping)
		{
			while (!ObjectsWithSolutions.IsEmpty())
			{
				// Avoid starvation. Find index of the actor that has the least number of options remaining, i.e. is the most constrained.
				// Example: OriginA could be remapped to TargetA or TargetB, OriginB can only be remapped to TargetB.
				// We should pick an option for OriginB first. If instead we did OriginA first and unfortunately chose TargetB, OriginB would be
				// left with 0 alternatives!
				// Btw: if anything is left unassigned using this approach, then there was no way to satisfy all actors in the first place.
				const int32 NextIndex = GetMostConstrainedIndex();
				
				const FActorWithSolutions& ObjectRemapData = ObjectsWithSolutions[NextIndex];
				const FSoftObjectPath RemappedToActor = PickSolutionAndRemap(ObjectRemapData, ProcessRemapping);
				ObjectsWithSolutions.RemoveAtSwap(NextIndex, EAllowShrinking::No);
				
				if (!RemappedToActor.IsNull())
				{
					RemoveFromSolutionSpace(RemappedToActor);
				}
			}
		}

		/** Finds the index of an actor that has the least amount of options left. */
		int32 GetMostConstrainedIndex() const
		{
			int32 CurrentLowest = TNumericLimits<int32>::Max();
			int32 Result = 0;
			for (int32 i = 0; i < ObjectsWithSolutions.Num(); ++i)
			{
				const int32 NumRemaining = ObjectsWithSolutions[i].PossibleSolutions.Num();
				if (NumRemaining == 1)
				{
					return i;
				}
				if (NumRemaining < CurrentLowest)
				{
					CurrentLowest = NumRemaining;
					Result = i;
				}
			}
			return Result;
		}

		/** Update ObjectsWithSolutions, removing TargetActor from the solution pool. Entries without solutions are removed.*/
		void RemoveFromSolutionSpace(const FSoftObjectPath& TargetActor)
		{
			for (int32 i = 0; i < ObjectsWithSolutions.Num(); ++i)
			{
				TSmallInlineArray<FSoftObjectPath>& PossibleSolutions = ObjectsWithSolutions[i].PossibleSolutions;
				if (!PossibleSolutions.Contains(TargetActor))
				{
					continue;
				}
				
				if (PossibleSolutions.Num() == 1)
				{
					ObjectsWithSolutions.RemoveAtSwap(i);
				}
				else
				{
					PossibleSolutions.RemoveSingle(TargetActor);
				}
			}
		}

		/**
		 * Picks the first solution, and replaces the original actor path to the new actor path.
		 * Subobject paths are translated with the new actor name.
		 */
		template<CProcessRemappingCallbable TProcessRemappingLambda>
		FSoftObjectPath PickSolutionAndRemap(const FActorWithSolutions& ObjectRemapData, TProcessRemappingLambda&& ProcessRemapping) const
		{
			if (ObjectRemapData.PossibleSolutions.IsEmpty())
			{
				return {};
			}

			const FSoftObjectPath& OriginActor = ObjectRemapData.OriginActor;
			const FSoftObjectPath& PickedSolution = ObjectRemapData.PossibleSolutions[0];

			// BuildOriginObjectHierarchyAndActorsNeedingRemapping added all replicated objects explicitly hence if an object is implicit, skip.
			const TOptional<EHierarchyObjectType> Hierarchy = ObjectPathHierarchy.IsInHierarchy(ObjectRemapData.OriginActor);
			const bool bIsReplicated = Hierarchy && *Hierarchy == EHierarchyObjectType::Explicit;
			if (bIsReplicated)
			{
				ProcessRemapping(OriginActor, PickedSolution);
			}
			
			RemapActorHierarchy(OriginActor, PickedSolution, ProcessRemapping);
			
			return PickedSolution;
		}

		/** For each original component that had replicated properties, replace the actor bit of their path with the chosen solution actor. */
		template<CProcessRemappingCallbable TProcessRemappingLambda>
		void RemapActorHierarchy(
			const FSoftObjectPath& OriginActor,
			const FSoftObjectPath& PickedSolution,
			TProcessRemappingLambda&& ProcessRemapping
			) const
		{
			ObjectPathHierarchy.TraverseTopToBottom([this, &ProcessRemapping, &PickedSolution](const FChildRelation& Relation)
			{
				const FSoftObjectPath& OriginalSubobject = Relation.Child.Object;
				// BuildOriginObjectHierarchyAndActorsNeedingRemapping added all replicated objects explicitly hence if an object is implicit, skip.
				const bool bNeedsRemapping = Relation.Child.Type == EHierarchyObjectType::Explicit;
				const TOptional<FSoftObjectPath> RemappedSubobjectPath = bNeedsRemapping
					? ReplaceActorInPath(OriginalSubobject, PickedSolution) : TOptional<FSoftObjectPath>{};
				
				if (RemappedSubobjectPath)
				{
					ProcessRemapping(OriginalSubobject, *RemappedSubobjectPath);
				}
				
				return ETreeTraversalBehavior::Continue;
			}, OriginActor);
		}
	};
}