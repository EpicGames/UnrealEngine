// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ObjectPathHierarchy.h"

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"

#define UE_API CONCERTSYNCCORE_API

struct FConcertObjectInStreamID;
struct FConcertReplicatedObjectId;
struct FConcertReplication_ChangeStream_Request;
struct FConcertReplicationStream;
struct FConcertReplication_Join_Request;

namespace UE::ConcertSyncCore
{
	/**
	 * Holds the outer hierarchy of all objects registered in any stream.
	 * Updates the hierarchy when a relevant event changing stream content happens.
	 */
	class FReplicatedObjectHierarchyCache : FObjectPathHierarchy
	{
	public:

		// AddObject and RemoveObject are not supposed to be invoked - use the dedicated OnX events below.
		using FObjectPathHierarchy::TraverseTopToBottom;
		using FObjectPathHierarchy::TraverseBottomToTop;
		using FObjectPathHierarchy::IsInHierarchy;
		using FObjectPathHierarchy::HasChildren;
		using FObjectPathHierarchy::IsEmpty;

		/** Removes all saved state. */
		UE_API void Clear();

		/**
		 * Checks whether the object is registered by any client (except for this in IgnoredClients).
		 *
		 * This function ignores implicit knowledge of the hierarchy.
		 * For example if you register ONLY /Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0, then
		 * - IsObjectReferencedDirectly(/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0) == true,
		 * - IsObjectReferencedDirectly(/Game/Maps.Map:PersistentLevel.Cube) == false
		 * but e.g. TraverseTopToBottom would list both paths.
		 */
		UE_API bool IsObjectReferencedDirectly(const FSoftObjectPath& ObjectPath, TConstArrayView<FGuid> IgnoredClients = {}) const;

		// Validated network events that modify the known hierarchy of objects.
		UE_API void OnJoin(const FGuid& ClientId, const FConcertReplication_Join_Request& Request);
		UE_API void OnPostClientLeft(const FGuid& ClientId, TConstArrayView<FConcertReplicationStream> Streams);
		UE_API void OnChangeStreams(const FGuid& ClientId, TConstArrayView<FConcertObjectInStreamID> AddedObjects, TConstArrayView<FConcertObjectInStreamID> RemovedObjects);

	private:

		struct FStreamReferencer
		{
			FGuid ClientId;
			FGuid StreamId;

			friend bool operator==(const FStreamReferencer& Left, const FStreamReferencer& Right) { return Left.ClientId == Right.ClientId && Left.StreamId == Right.StreamId; }
			friend bool operator!=(const FStreamReferencer& Left, const FStreamReferencer& Right) { return !(Left == Right); }
		};
		
		struct FObjectMetaData
		{
			/** Keeps track of all clients referencing the stream. */
			TArray<FStreamReferencer> ReferencingStreams;
		};
		
		TMap<FSoftObjectPath, FObjectMetaData> ObjectMetaData;

		UE_API void AddObjectInternal(const FConcertReplicatedObjectId& ObjectInfo);
		UE_API void RemoveObjectInternal(const FConcertReplicatedObjectId& Object);
		UE_API bool RemoveMetaData(const FConcertReplicatedObjectId& Object);
	};
}

#undef UE_API
