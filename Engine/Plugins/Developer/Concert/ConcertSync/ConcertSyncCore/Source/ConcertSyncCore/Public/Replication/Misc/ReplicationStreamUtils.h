// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/ReplicationStream.h"
#include "Misc/ObjectPathOuterIterator.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "UObject/SoftObjectPath.h"

#include <type_traits>

namespace UE::ConcertSyncCore
{
	/** Util for finding a stream in an array of streams by id. */
	const FConcertReplicationStream* FindStream(const TConstArrayView<FConcertReplicationStream> Streams, const FGuid& StreamId);
	/** Util for finding a stream in an array of streams by id. */
	FConcertReplicationStream* FindStreamEditable(const TArrayView<FConcertReplicationStream> Streams, const FGuid& StreamId);
	
	/** Util for finding an object in an array of streams. */
	const FConcertReplicatedObjectInfo* FindObjectInfo(const TConstArrayView<FConcertReplicationStream> Streams, const FConcertObjectInStreamID& ObjectId);
	/** Util for finding an object an array of streams. */
	FConcertReplicatedObjectInfo* FindObjectInfoEditable(const TArrayView<FConcertReplicationStream> Streams, const FConcertObjectInStreamID& ObjectId);

	/** Util for finding an object in a stream. */
	const FConcertReplicatedObjectInfo* FindObjectInfo(const FConcertReplicationStream& Stream, const FSoftObjectPath& ObjectPath);
	/** Util for finding an object in a stream. */
	FConcertReplicatedObjectInfo* FindObjectInfoEditable(FConcertReplicationStream& Stream, const FSoftObjectPath& ObjectPath);
	
	/** Util for finding an object's frequency override settings in a stream in an array of streams. */
	const FConcertObjectReplicationSettings* FindObjectFrequency(const TConstArrayView<FConcertReplicationStream> Streams, const FConcertObjectInStreamID& ObjectId);
	/** Util for finding an object's frequency override settings in a stream in an array of streams. */
	FConcertObjectReplicationSettings* FindObjectFrequencyEditable(const TArrayView<FConcertReplicationStream> Streams, const FConcertObjectInStreamID& ObjectId);
	
	/** @return Whether ObjectPath or any its child objects, i.e. any object that ObjectPath is an indirect or direct outer of, are referenced by Streams. */
	bool IsObjectOrChildReferenced(const TConstArrayView<FConcertReplicationStream> Streams, const FSoftObjectPath& ObjectPath);
}

namespace UE::ConcertSyncCore
{
	inline const FConcertReplicationStream* FindStream(const TConstArrayView<FConcertReplicationStream> Streams, const FGuid& StreamId)
	{
		return Streams.FindByPredicate([&StreamId](const FConcertReplicationStream& Stream) { return Stream.BaseDescription.Identifier == StreamId; });
	}
	inline FConcertReplicationStream* FindStreamEditable(const TArrayView<FConcertReplicationStream> Streams, const FGuid& StreamId)
	{
		return Streams.FindByPredicate([&StreamId](const FConcertReplicationStream& Stream) { return Stream.BaseDescription.Identifier == StreamId; });
	}
	
	inline const FConcertReplicatedObjectInfo* FindObjectInfo(const TConstArrayView<FConcertReplicationStream> Streams, const FConcertObjectInStreamID& ObjectId)
	{
		const FConcertReplicationStream* Stream = FindStream(Streams, ObjectId.StreamId);
		return Stream ? Stream->BaseDescription.ReplicationMap.ReplicatedObjects.Find(ObjectId.Object) : nullptr;
	}
	inline FConcertReplicatedObjectInfo* FindObjectInfoEditable(const TArrayView<FConcertReplicationStream> Streams, const FConcertObjectInStreamID& ObjectId)
	{
		FConcertReplicationStream* Stream = FindStreamEditable(Streams, ObjectId.StreamId);
		return Stream ? Stream->BaseDescription.ReplicationMap.ReplicatedObjects.Find(ObjectId.Object) : nullptr;
	}
	
	inline const FConcertReplicatedObjectInfo* FindObjectInfo(const FConcertReplicationStream& Stream, const FSoftObjectPath& ObjectPath)
	{
		return Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Find(ObjectPath);
	}
	inline FConcertReplicatedObjectInfo* FindObjectInfoEditable(FConcertReplicationStream& Stream, const FSoftObjectPath& ObjectPath)
	{
		return Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Find(ObjectPath);
	}
	
	inline const FConcertObjectReplicationSettings* FindObjectFrequency(const TConstArrayView<FConcertReplicationStream> Streams, const FConcertObjectInStreamID& ObjectId)
	{
		const FConcertReplicationStream* Stream = FindStream(Streams, ObjectId.StreamId);
		return Stream ? Stream->BaseDescription.FrequencySettings.ObjectOverrides.Find(ObjectId.Object) : nullptr;
	}
	inline FConcertObjectReplicationSettings* FindObjectFrequencyEditable(const TArrayView<FConcertReplicationStream> Streams, const FConcertObjectInStreamID& ObjectId)
	{
		FConcertReplicationStream* Stream = FindStreamEditable(Streams, ObjectId.StreamId);
		return Stream ? Stream->BaseDescription.FrequencySettings.ObjectOverrides.Find(ObjectId.Object) : nullptr;
	}

	inline bool IsObjectOrChildReferenced(const TConstArrayView<FConcertReplicationStream> Streams, const FSoftObjectPath& ObjectPath)
	{
		for (const FConcertReplicationStream& Stream : Streams)
		{
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				for (FObjectPathOuterIterator It(Pair.Key); It; ++It)
				{
					if (*It == ObjectPath)
					{
						return true;
					}
				}
			}
		}
		return false;
	}
}
