// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Messages/RestoreContent.h"

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Templates/FunctionFwd.h"

struct FConcertBaseStreamInfo;
struct FConcertClientInfo;
struct FConcertObjectInStreamID;
struct FConcertSessionActivity;
struct FConcertSyncActivity;
struct FConcertSyncReplicationEvent;

template<typename OptionalType> struct TOptional;

namespace UE::ConcertSyncCore::Replication
{
	using FExtractReplicationEventFunc = TFunctionRef<void(
		const int64 EventId,
		TFunctionRef<void(const FConcertSyncReplicationEvent& Event)> Callback
		)>;
	
	/**
	 * Walks back the activity history and finds the latest activity that sets the target client's state.
	 *
	 * For now, the state is only determined by EConcertSyncReplicationActivityType::LeaveReplication replication activities.
	 * In the future, additional types may also affect it, e.g. PutState, may then each in turn produce activities.
	 *
	 * @param Activities The activities to analyze
	 * @param IsTargetEndpointFunc Returns whether the endpoint ID corresponds to the client to which we want to get the state.
	 *	Usually when restoring, you'll want to match the client's DisplayName and DeviceName to that in an activity.
	 * @param GetReplicationEventFunc Gets the replication event for the given EventId. This is called when iterating Activities.
	 * @param OutStreams The streams the client had, if successful
	 * @param OutAuthority The authority the client had, if successful
	 *
	 * @return The activity ID that contained the state. Unset if not found.
	 */
	CONCERTSYNCCORE_API TOptional<int64> BacktrackActivityHistoryForActivityThatSetsContent(
		TConstArrayView<FConcertSyncActivity> Activities,
		TFunctionRef<bool(const FGuid& EndpointId)> IsTargetEndpointFunc,
		FExtractReplicationEventFunc GetReplicationEventFunc,
		TArray<FConcertBaseStreamInfo>& OutStreams,
		TArray<FConcertObjectInStreamID>& OutAuthority
		);
	/** Equivalent version that accepts FConcertSessionActivity instead. This is useful if your activities come from IConcertClientWorkspace::GetActivities. */
	CONCERTSYNCCORE_API TOptional<int64> BacktrackActivityHistoryForActivityThatSetsContent(
		TConstArrayView<FConcertSessionActivity> Activities,
		TFunctionRef<bool(const FGuid& EndpointId)> IsTargetEndpointFunc,
		FExtractReplicationEventFunc GetReplicationEventFunc,
		TArray<FConcertBaseStreamInfo>& OutStreams,
		TArray<FConcertObjectInStreamID>& OutAuthority
		);
	
	/**
	 * Decides whether First and Second should be considered to represent the same user across several Concert sessions.
	 *
	 * Every time a user joins a Concert session, a new endpoint ID is generated for that user and saved in the database.
	 * Even though the endpoint ID is different, we can associate the same user across the IDs by using the DisplayName and DeviceName.
	 *
	 * @return Whether First and Second logically describe the same client (i.e. DisplayName and DeviceName of both clients are equal).
	 */
	CONCERTSYNCCORE_API bool AreLogicallySameClients(const FConcertClientInfo& First, const FConcertClientInfo& Second);
}
