// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ObjectIds.h"
#include "Replication/Messages/SyncControl.h"

#include "Containers/Map.h"
#include "Misc/EnumClassFlags.h"

namespace UE::ConcertSyncCore
{
	enum class EAppendSyncControlFlags : uint8
	{
		None,
		/** The resulting SyncControlToUpdate should not contain any false values. */
		SkipLostControl = 1 << 0
	};
	ENUM_CLASS_FLAGS(EAppendSyncControlFlags);

	/** Appends AppendedSyncControl to SyncControlToUpdate overriding old entries. */
	void AppendSyncControl(FConcertReplication_ChangeSyncControl& SyncControlToUpdate, const FConcertReplication_ChangeSyncControl& AppendedSyncControl, EAppendSyncControlFlags Flags = EAppendSyncControlFlags::None);
}

namespace UE::ConcertSyncCore
{
	inline void AppendSyncControl(FConcertReplication_ChangeSyncControl& SyncControlToUpdate, const FConcertReplication_ChangeSyncControl& AppendedSyncControl, EAppendSyncControlFlags Flags)
	{
		const bool bShouldSkipLostControl = EnumHasAnyFlags(Flags, EAppendSyncControlFlags::SkipLostControl);
		for (const TPair<FConcertObjectInStreamID, bool>& Change : AppendedSyncControl.NewControlStates)
		{
			if (bShouldSkipLostControl && !Change.Value)
			{
				// Do not append the change but more importantly, also remove any potential true state from SyncControlToUpdate.
				SyncControlToUpdate.NewControlStates.Remove(Change.Key);
				continue;
			}
			
			SyncControlToUpdate.NewControlStates.Add(Change.Key, Change.Value);
		}
	}
}
