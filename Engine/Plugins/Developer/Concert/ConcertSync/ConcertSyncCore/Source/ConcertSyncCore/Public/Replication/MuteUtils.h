// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Optional.h"

struct FConcertReplication_ObjectMuteSetting;
struct FConcertReplication_ChangeMuteState_Request;
struct FSoftObjectPath;

namespace UE::ConcertSyncCore::Replication::MuteUtils
{
	enum class EMuteState : uint8
	{
		/** Default state: the object was not explicitly muted / unmuted, nor was any of its parent objects affected. */
		None,
		
		ExplicitlyMuted,
		ExplicitlyUnmuted,
		ImplicitlyMuted,
		ImplicitlyUnmuted
	};

	/** Holds information that is used to rebuilding mute state. */
	class IMuteStateGroundTruth
	{
	public:

		/** @return The object's mute state. */
		virtual EMuteState GetMuteState(const FSoftObjectPath& Object) const = 0;

		/** @return The mute setting set for Object, if GetMuteState(Object) == ExplicitlyMuted or ExplicitlyUnmuted. Unset otherwise. */
		virtual TOptional<FConcertReplication_ObjectMuteSetting> GetExplicitSetting(const FSoftObjectPath& Object) const = 0;

		// TODO DP: add inline GetImplicitSetting

		/**
		 * Checks whether an object is known.
		 * 
		 * Only known objects can be muted on the server.
		 * @see FConcertReplication_ChangeMuteState_Request
		 * 
		 * @return Whether the object is known on the server.
		 */
		virtual bool IsObjectKnown(const FSoftObjectPath& Object) const = 0;

		virtual ~IMuteStateGroundTruth() = default;
	};

	/**
	 * Adds a request into a base request, which gets its data overriden accordingly.
	 *
	 * This is useful for replaying mute requests.
	 * For example, suppose the base request mutes Foo and InRequestToMerge unmutes it, the final result would be that Foo is not muted.
	 *
	 * @param InOutBase The result will be combined into this request.
	 * @param InRequestToMerge The request to add onto InOutBase.
	 * @param InGroundTruth Used for determining whether an object can be muted.
	 */
	CONCERTSYNCCORE_API void CombineMuteRequests(
		FConcertReplication_ChangeMuteState_Request& InOutBase,
		const FConcertReplication_ChangeMuteState_Request& InRequestToMerge,
		const IMuteStateGroundTruth& InGroundTruth
		);
}
