// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertReplicationAction.generated.h"

struct FConcertReplicatedObjectId;

namespace UE::ConcertSyncCore
{
/** Arguments for FConcertReplicationAction::Apply. */
struct FReplicationActionArgs
{
	/** Replication info about the object. */
	const FConcertReplicatedObjectId& ObjectId;
	
	/** The resolved object that was replicated. */
	const TNotNull<UObject*> Object;

	explicit FReplicationActionArgs(const FConcertReplicatedObjectId& InObjectId, TNotNull<UObject*> InObject)
		: ObjectId(InObjectId)
		, Object(InObject)
	{}
};
}

/**
 * An action to be performed in relation to replication, such as after an object has been replicated.
 * 
 * Sub-structs implement Apply, which e.g. calls PostEditChange, MarkRenderStateDirty, or some other custom action.
 * This can be used in conjunction with TInstancedStruct so these actions can be set up in .ini files,
 * i.e. UPROPERTY(Config) TInstancedStruct<FConcertReplicationAction>.
 */
USTRUCT()
struct FConcertReplicationAction
{
	GENERATED_BODY()

	virtual ~FConcertReplicationAction() = default;
	virtual void Apply(const UE::ConcertSyncCore::FReplicationActionArgs& InArgs) const { checkNoEntry(); }
};
