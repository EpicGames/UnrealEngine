// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertReplicationAction.h"
#if WITH_ENGINE
#include "Components/SceneComponent.h"
#endif
#include "ConcertReplicationAction_MarkRenderStateDirty.generated.h"

/** Calls MarkRenderStateDirty on the object if it is a USceneComponent. */
USTRUCT(DisplayName = "Mark Render State Dirty")
struct FConcertReplicationAction_MarkRenderStateDirty : public FConcertReplicationAction
{
	GENERATED_BODY()

	virtual void Apply(const UE::ConcertSyncCore::FReplicationActionArgs& InArgs) const override
	{
#if WITH_ENGINE
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(InArgs.Object))
		{
			SceneComponent->MarkRenderStateDirty();
		}
#endif
	}
};
