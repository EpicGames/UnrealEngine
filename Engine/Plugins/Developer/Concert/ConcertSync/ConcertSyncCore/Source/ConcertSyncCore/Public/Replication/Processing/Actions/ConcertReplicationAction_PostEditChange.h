// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertReplicationAction.h"
#include "UObject/Object.h"
#include "ConcertReplicationAction_PostEditChange.generated.h"

/** Calls PostEditChange on the object. */
USTRUCT(DisplayName = "Post Edit Change")
struct FConcertReplicationAction_PostEditChange : public FConcertReplicationAction
{
	GENERATED_BODY()

	virtual void Apply(const UE::ConcertSyncCore::FReplicationActionArgs& InArgs) const override
	{
#if WITH_EDITOR
		InArgs.Object->PostEditChange();
#endif
	}
};
