// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertReplicationAction.h"
#if WITH_ENGINE
#include "Components/SceneComponent.h"
#endif
#include "ConcertReplicationAction_UpdateComponentToWorld.generated.h"

/** Calls ConcertReplicationAction_UpdateComponentToWorld on the object if it is a USceneComponent. */
USTRUCT(DisplayName = "Update Component To World")
struct FConcertReplicationAction_UpdateComponentToWorld : public FConcertReplicationAction
{
	GENERATED_BODY()

	virtual void Apply(const UE::ConcertSyncCore::FReplicationActionArgs& InArgs) const override
	{
#if WITH_ENGINE
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(InArgs.Object))
		{
			SceneComponent->UpdateComponentToWorld();
		}
#endif
	}
};
