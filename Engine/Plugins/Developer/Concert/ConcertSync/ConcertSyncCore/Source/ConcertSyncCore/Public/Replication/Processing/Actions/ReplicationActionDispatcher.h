// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"

#define UE_API CONCERTSYNCCORE_API

struct FConcertReplicationActionEntry;

namespace UE::ConcertSyncCore
{
struct FReplicationActionArgs;

/** Util that you can use when calling IObjectReplicationFormat::ApplyReplicationEvent to execute actions. */
class FReplicationActionDispatcher
{
public:

	UE_API explicit FReplicationActionDispatcher(const TArray<FConcertReplicationActionEntry>& InActions UE_LIFETIMEBOUND, bool bDebugActions = false);

	/** Checks whether InProperty should trigger actions. */
	UE_API void OnReplicateProperty(const FProperty& InProperty);

	/** Call all properties have been processed using OnReplicateProperty. Triggers the actions that need to triggered.  */
	UE_API void ExecuteActions(const FReplicationActionArgs& InArgs);

private:

	/** The actions to perform. */
	const TArray<FConcertReplicationActionEntry>& Actions;

	/** Whether the actions should be debugged. */
	const bool bDebugActions;

	/** Bit mask of equal length as the action array. Each index marks whether the action at the equivalent action array index should be performed. */
	TBitArray<> ActionsToTrigger;
};
}

#undef UE_API
