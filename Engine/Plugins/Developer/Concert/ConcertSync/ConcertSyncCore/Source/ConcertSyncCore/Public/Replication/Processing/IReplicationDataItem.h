// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

struct FConcertSessionSerializedPayload;

namespace UE::ConcertSyncCore
{
/** An item that is cached by IReplicationDataSource. Provides interface for extracting the replication data.*/
class IReplicationDataItem
{
public:
	
	/**
	 * Extracts data for an object iterated by IReplicationDataSource.
	 * 
	 * This can update the SequenceId associated with the object (if this data source generates data as opposed to queuing it).
	 * Either ProcessCopyable or ProcessMoveable will be called, never both, and it will be called at most once.
	 * 
	 * @param ProcessCopyable Callback if the event was retrieved and not owned by this IReplicationDataSource (hence not being moveable).
	 * @param ProcessMoveable Callback if the event was just constructed (and hence can be moved)	
	 */
	virtual void ExtractReplicationDataForObject(
		TFunctionRef<void(const FConcertSessionSerializedPayload& Payload)> ProcessCopyable,
		TFunctionRef<void(FConcertSessionSerializedPayload&& Payload)> ProcessMoveable
		) = 0;
	
	/** Util version for callers that only want to read and do not want to store the payload. */
	void ExtractReplicationDataForObject(TFunctionRef<void(const FConcertSessionSerializedPayload& Payload)> ProcessCopyable)
	{
		ExtractReplicationDataForObject(ProcessCopyable, [&ProcessCopyable](FConcertSessionSerializedPayload&& Payload){ ProcessCopyable(Payload); });
	}
	
	virtual ~IReplicationDataItem() = default;
};
}
