// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/SequenceId.h"

#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"

namespace UE::ConcertSyncCore
{
	class IReplicationDataItem;

	struct FPendingObjectReplicationInfo
	{
		/** The object to be replicated */
		FConcertReplicatedObjectId ObjectId;
		/**
		 * The ID of the change that is pending.
		 * If previous changes were combined, this is the ID of the highest combined change.
		 */
		FSequenceId SequenceId;
	};
	
	/**
	 * Responsible for obtaining replication data.
	 * 
	 * Implementation examples:
	 * - serialize the object path (client),
	 * - cache received object data and return it (client & server)
	 */
	class IReplicationDataSource
	{
	public:
		
		/**
		 * Iterates the objects that must be processed for replication.
		 * 
		 * It allows the extraction of the data using IReplicationDataItem. If the data is extracted, the data is "dequeued", i.e. will not be 
		 * passed again in subsequent calls to ConsumePendingObjects unless new data becomes available.
		 */
		virtual void ConsumePendingObjects(TFunctionRef<void(const FPendingObjectReplicationInfo& Info, IReplicationDataItem& Item)> InProcess) = 0;
		
		/** Iterates the objects that must be processed for replication (the result of ExtractReplicationDataForObject MAY return something new). */
		UE_DEPRECATED(5.8, "Use ConsumePendingObjects instead.")
		virtual void ForEachPendingObject(TFunctionRef<void(const FPendingObjectReplicationInfo&)> ProcessItemFunc) const {}
		
		/** @return The number of objects ForEachPendingObject would iterate. Can be used, e.g. for Reserve()-ing a container. */
		UE_DEPRECATED(5.8, "This function no longer has a use and will be removed.")
		virtual int32 NumObjects() const { return 0; }
		/**
		 * Extracts data for Object. Object must have been previously returned by ForEachPendingObject.
		 * 
		 * This "dequeues" Object so it will not be mentioned by ForEachPendingObject again until it is again marked as "dirty".
		 * This might also update the SequenceId associated with the object (if this data source generates data as opposed to queuing it).
		 *
		 * The call to ProcessCopyable / ProcessMoveable may be skipped if there is no new data to send.
		 * Either ProcessCopyable or ProcessMoveable will be called, never both, and it will be called at most once.
		 * @param Object The object for which to obtain data
		 * @param ProcessCopyable Callback if the event was retrieved and not owned by this IReplicationDataSource (hence not being moveable).
		 * @param ProcessMoveable Callback if the event was just constructed (and hence can be moved)
		 * 
		 * @return Whether successful. False indicates the call was invalid to make (ForEachPendingObject did not return Object). True indicates success, even if ProcessCopyable was not called.		
		 */
		UE_DEPRECATED(5.8, "Use ConsumePendingObjects and call IReplicationDataItem::ExtractReplicationDataForObject instead.")
		virtual bool ExtractReplicationDataForObject(
			const FConcertReplicatedObjectId& Object,
			TFunctionRef<void(const FConcertSessionSerializedPayload& Payload)> ProcessCopyable,
			TFunctionRef<void(FConcertSessionSerializedPayload&& Payload)> ProcessMoveable
			)
		{
			return false;
		}
		
		/** Util version for callers that only want to read and do not want to store the payload. */
		UE_DEPRECATED(5.8, "Use ConsumePendingObjects and call IReplicationDataItem::ExtractReplicationDataForObject instead.")
		bool ExtractReplicationDataForObject(const FConcertReplicatedObjectId& Object, TFunctionRef<void(const FConcertSessionSerializedPayload& Payload)> ProcessCopyable)
		{
			return false;
		}
		
		virtual ~IReplicationDataSource() = default;
	};
}
