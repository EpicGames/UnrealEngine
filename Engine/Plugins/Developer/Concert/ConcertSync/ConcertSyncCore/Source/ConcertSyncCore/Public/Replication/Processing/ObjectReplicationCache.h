// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/SequenceId.h"

#include "Containers/Array.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#define UE_API CONCERTSYNCCORE_API

struct FConcertReplication_ObjectReplicationEvent;
struct FSoftObjectPath;
struct FGuid;

namespace UE::ConcertSyncCore
{
	class IObjectReplicationFormat;

	/**
	 * Interface for objects that want to latently use received FConcertObjectReplicationEvent data.
	 * The data is continuously updated until it is consumed.
	 */
	class IReplicationCacheUser
	{
	public:

		/** @return Whether this user is interested in data from this object. */
		virtual bool WantsToAcceptObject(const FConcertReplicatedObjectId& Object) const = 0;

		/**
		 * Called when data that is interesting to this user becomes available.
		 *
		 * The user can keep hold of Data until it is used, at which point it just let's Data get out of scope.
		 * If new data is received while this user is referencing Data, the Data will be combined and OnCachedDataUpdated called.
		 */
		virtual void OnDataCached(const FConcertReplicatedObjectId& Object, FSequenceId SequenceId, TSharedRef<const FConcertReplication_ObjectReplicationEvent> Data) = 0;

		/**
		 * This function is called when new data is received while this cache user is still holding on to an object previously received with OnDataCached.
		 * @param Object The object for which data was updated.
		 * @param SequenceId The ID of the event data that was combined
		 */
		virtual void OnCachedDataUpdated(const FConcertReplicatedObjectId& Object, FSequenceId SequenceId) {}

		virtual ~IReplicationCacheUser() = default;
	};
	
	struct FCacheStoreStats
	{
		uint32 NumInsertions = 0;
		uint32 NumCacheUpdates = 0;

		bool NoChangesMade() const { return NumInsertions == 0 && NumCacheUpdates == 0; }
	};
	
	/**
	 * This is an intermediate place for received data to live before it is further processed.
	 *
	 * IObjectCacheUsers register with the cache and decide which data is to be received.
	 * When replication data comes in IObjectCacheUsers::WantsToAcceptObject is used to determine whether the object wants the data.
	 * If the data should be received, IObjectCacheUsers::OnDataCached is called receiving a shared ptr to the data.
	 * When the data is finally consumed latently, e.g. sent to other endpoints, the cache user resets the shared ptr.
	 * If new data comes in before a cache user consumes it, the new data and old data are combined (using IObjectReplicationFormat::CombineReplicationEvents). 
	 *
	 * This allows multiple systems to reuse replication data. For example, on the server the same data may need to be distributed to different clients but
	 * the clients send the data at different times.
	 */
	class FObjectReplicationCache : public TSharedFromThis<FObjectReplicationCache>
	{
	public:

		UE_API FObjectReplicationCache(IObjectReplicationFormat& ReplicationFormat UE_LIFETIMEBOUND);
		
		/**
		 * Called when new data is received for an object and shares it with any IObjectCacheUser that is possibly interested in it.
		 * @param SendingEndpointId The ID of the client endpoint that sent this data
		 * @param OriginStreamId The stream from which the object was replicated
		 * @param SequenceId The ID of the change. Used primarily for performance tracing.
		 * @param ObjectReplicationEvent The data that was replicated
		 * @return The number of cache users that accepted this event. 
		 */
		UE_API FCacheStoreStats StoreUntilConsumed(const FGuid& SendingEndpointId, const FGuid& OriginStreamId, const FSequenceId SequenceId, const FConcertReplication_ObjectReplicationEvent& ObjectReplicationEvent);

		/** Registers a new user, which will start receiving data for any new data received from now on. */
		UE_API void RegisterDataCacheUser(TSharedRef<IReplicationCacheUser> User);
		UE_API void UnregisterDataCacheUser(const TSharedRef<IReplicationCacheUser>& User);

	private:

		/** Used for combining events to save network bandwidth. */
		IObjectReplicationFormat& ReplicationFormat;

		/** Everyone who registered for receiving data. */
		TArray<TSharedRef<IReplicationCacheUser>> CacheUsers;

		struct FObjectCache
		{
			/**
			 * Past data that is still in use by users.
			 * 
			 * StoreUntilConsumed asks all IReplicationCacheUser and if at least one is interested, creates exactly 1 TSharedRef<FConcertObjectReplicationEvent>.
			 * Every IReplicationCacheUser receives essentially a MakeShareable of the above instance: its own TSharedRef with a custom deleter that simply keeps the event instance.
			 *
			 * This mechanism allows detecting whether a IReplicationCacheUser already is using old data which needs to be combined with the new incoming data, or needs a new instance.
			 * The intention is that as soon as IReplicationCacheUser has finished using a data event, it will receive a
			 * new instance: we do not want IReplicationCacheUsers to have large histories of events being combined into them.
			 */
			TMap<TWeakPtr<IReplicationCacheUser>, TWeakPtr<FConcertReplication_ObjectReplicationEvent>> DataInUse;
		};
		/** Maps every object to the events cached for it. */
		TMap<FConcertObjectInStreamID, FObjectCache> Cache;
		
		/**
		 * Combines old data that cache users may have associated for ObjectId with the NewData and notifies the users with OnCachedDataUpdated.
		 * @return Number of cache users that were updated
		 */
		UE_API uint32 CombineCachedDataWithNewData(
			const FConcertReplicatedObjectId& ObjectId,
			FSequenceId NewSequenceId,
			const FConcertReplication_ObjectReplicationEvent& NewData,
			const FObjectCache& ObjectCacheBeforeAddition
			) const;
	};
}

#undef UE_API
