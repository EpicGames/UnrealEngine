// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IReplicationDataSource.h"
#include "Replication/Data/ObjectIds.h"

#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPath.h"

#define UE_API CONCERTSYNCCORE_API

struct FConcertObjectReplicationSettings;
struct FConcertObjectInStreamID;

namespace UE::ConcertSyncCore
{
	class IReplicationDataSource;
		
	/**
	 * Params for FObjectReplicationProcessor::ProcessObjects.
	 * Exists as struct so we can add params (e.g. UE-190714) without affecting the signature and can avoid deprecation.
	 */
	struct FProcessObjectsParams
	{
		/** The current frame's delta time */
		float DeltaTime;
		// TODO UE-190714: We should add a time budget.
	};
	
	/**
	 * Responsible for prioritizing a list of objects and processing them.
	 * 
	 * Processing is defined by the subclass. Some example implementations:
	 *  - serialize the object and send (client),
	 *  - find loaded object and apply replication data (client),
	 *  - send object data to interested clients (server)
	 *  
	 * FObjectReplicationProcessor is backed by an IReplicationDataSource, which is a source of replicated object data.
	 */
	class FObjectReplicationProcessor
	{
	public:
		
		/**
		 * @param DataSource Source of the data that is to be sent
		 */
		UE_API FObjectReplicationProcessor(IReplicationDataSource& DataSource UE_LIFETIMEBOUND);
		virtual ~FObjectReplicationProcessor() = default;
		
		UE_API virtual void ProcessObjects(const FProcessObjectsParams& Params);
		
	protected:

		struct FObjectProcessArgs
		{
			/** Info about the object to process */
			FPendingObjectReplicationInfo ObjectInfo;
			
			/** Item you can use to extract the replication info from. */
			IReplicationDataItem& Item;

			explicit FObjectProcessArgs(FPendingObjectReplicationInfo InObjectInfo, IReplicationDataItem& InItem)
				: ObjectInfo(MoveTemp(InObjectInfo)) , Item(InItem)
			{}
		};

		FORCEINLINE IReplicationDataSource& GetDataSource() const { return DataSource; }

		/** Processes the object. */
		virtual void ProcessObject(const FObjectProcessArgs& Args) = 0;

	private:
		
		/** Abstracts where replication data comes from: could be generated (clients) or received (server or client) */
		IReplicationDataSource& DataSource;
	};
}

#undef UE_API
