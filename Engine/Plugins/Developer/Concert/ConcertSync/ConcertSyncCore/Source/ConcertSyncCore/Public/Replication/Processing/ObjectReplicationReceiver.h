// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API CONCERTSYNCCORE_API

class IConcertSession;
struct FConcertReplication_BatchReplicationEvent;
struct FConcertReplication_ObjectReplicationEvent;
struct FConcertSessionContext;
struct FConcertReplication_StreamReplicationEvent;

namespace UE::ConcertSyncCore
{
	class FObjectReplicationCache;
	
	/** Receives replicated object data from all message endpoints and stores it in a FObjectReplicationCache. */
	class FObjectReplicationReceiver
	{
	public:

		UE_API FObjectReplicationReceiver(IConcertSession& Session UE_LIFETIMEBOUND, FObjectReplicationCache& ReplicationCache UE_LIFETIMEBOUND);
		UE_API virtual ~FObjectReplicationReceiver();

	protected:

		/** Whether the object should be processed. */
		virtual bool ShouldAcceptObject(const FConcertSessionContext& SessionContext, const FConcertReplication_StreamReplicationEvent& StreamEvent, const FConcertReplication_ObjectReplicationEvent& ObjectEvent) const { return true; }

	private:

		/** The session that is being received on. */
		IConcertSession& Session;
		/** Where received data is stored. */
		FObjectReplicationCache& ReplicationCache;

		UE_API void HandleBatchReplicationEvent(const FConcertSessionContext& SessionContext, const FConcertReplication_BatchReplicationEvent& Event);
	};
}

#undef UE_API
