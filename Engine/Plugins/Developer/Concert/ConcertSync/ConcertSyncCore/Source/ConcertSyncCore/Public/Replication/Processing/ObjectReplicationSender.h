// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectReplicationProcessor.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Trace/ConcertTraceConfig.h"

#define UE_API CONCERTSYNCCORE_API

class IConcertSession;

namespace UE::ConcertSyncCore
{
	/**
	 * Sends object data to a specific endpoint ID.
	 */
	class FObjectReplicationSender : public FObjectReplicationProcessor
	{
	public:
		
		/**
		 * @param TargetEndpointId The endpoints to send to
		 * @param Session The session to use for sending
		 * @param DataSource Source of the data that is to be sent
		 */
		UE_API FObjectReplicationSender(
			const FGuid& TargetEndpointId,
			IConcertSession& Session UE_LIFETIMEBOUND, 
			IReplicationDataSource& DataSource UE_LIFETIMEBOUND
			);
		
		//~ Begin FObjectReplicationProcessor Interface
		UE_API virtual void ProcessObjects(const FProcessObjectsParams& Params) override;
		//~ End FObjectReplicationProcessor Interface

	protected:

		//~ Begin FObjectReplicationProcessor Interface
		UE_API virtual void ProcessObject(const FObjectProcessArgs& Args) override;
		//~ End FObjectReplicationProcessor Interface

	private:

		/** The endpoint data will be sent to */
		const FGuid TargetEndpointId;
		
		/** The session through which replication messages are sent. */
		IConcertSession& Session;

#if UE_CONCERT_TRACE_ENABLED
		/** Set by ProcessObject. Used when we actually start sending the data. */
		TMap<FConcertReplicatedObjectId, FSequenceId> ObjectsToTraceThisFrame;
#endif

		/** This event is filled in ProcessObjects and finally sent to TargetEndpointId. */
		FConcertReplication_BatchReplicationEvent EventToSend;

		UE_API void MarkObjectForTrace(const FConcertReplicatedObjectId& Object, FSequenceId Id);
		UE_API void TraceStartSendingMarkedObjects();
	};
}

#undef UE_API
