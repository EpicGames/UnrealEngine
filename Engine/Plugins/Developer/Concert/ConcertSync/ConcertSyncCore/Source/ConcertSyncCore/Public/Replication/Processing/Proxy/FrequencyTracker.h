// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/ReplicationFrequencySettings.h"

#include "Containers/Map.h"
#include "HAL/PlatformTime.h"

#define UE_API CONCERTSYNCCORE_API

struct FDateTime;

namespace UE::ConcertSyncCore
{
	/**
	 * Keeps track of when objects are processed.
	 * Allows querying of whether in a given frame an object is allowed to be processed (or whether it is too early).
	 *
	 * This keeps a TMap<FSoftObjectPath, FGuid> of objects that were processed in the past.
	 * You must periodically call CleanseOutdatedObjects() to avoid leaking objects that are no longer sent.
	 * @see FFrequencyTracker_CleanByTick
	 */
	class FFrequencyTracker
	{
	public:

		// Holds the return value of FPlatformTime::Seconds()
		using FPlatformSecondsTimestamp = double;

		/**
		 * Ensures ObjectPath is tracked in the cache and returns whether it can be processed now given its frequency settings.
		 *
		 * Note that multiple streams may be replicating the object and each stream has a different frequency settings for an object, e.g.
		 * object StaticMeshActor in stream A may be replicating RelativeLocation at 60 FPS and in stream B may be replicating RelativeRotation at 30 FPS (contrived example).
		 *
		 * The object can send immediately when it is first added, i.e. TrackAndCheckObject returns ECanProcessObject::CanProcess if the object was not yet registered.
		 * 
		 * @param ObjectPath The object that should be processed
		 * @param ReplicationSettings The frequency settings for the object.
		 * @param NowAsPlatformSeconds The current time - result of FPlatformTime::Seconds(). Pass this in to avoid excessive time spent on timing.
		 * 
		 * @return Whether the object can be processed now
		 */
		UE_API ECanProcessObject TrackAndCheckObject(
			const FConcertObjectInStreamID& ObjectPath,
			const FConcertObjectReplicationSettings& ReplicationSettings,
			const FPlatformSecondsTimestamp NowAsPlatformSeconds = FPlatformTime::Seconds()
			);

		/**
		 * Called when an object is processed.
		 * This is allowed to be called even if TrackAndCheckObject reported it being too early to process the object.
		 *
		 * @param Object The object that was processed
		 * @param NowAsPlatformSeconds The current time - result of FPlatformTime::Seconds(). Pass this in to avoid excessive time spent on timing.
		 */
		UE_API void OnProcessObject(const FConcertObjectInStreamID& Object, const FPlatformSecondsTimestamp NowAsPlatformSeconds = FPlatformTime::Seconds());
		
		/**
		 * Removes all tracked objects were last processed before CutoffTime.
		 * @param CutoffTime A past value of FPlatformTime::Seconds()
		 */
		UE_API void CleanseOutdatedObjects(const FPlatformSecondsTimestamp CutoffTime);

	protected:

		// For unit tests
		const TMap<FConcertObjectInStreamID, FPlatformSecondsTimestamp>& GetObjectsToLastProcessed() const { return ObjectsToLastProcessed; }
		
	private:

		/** It is important to call CleanseOutdatedObjects to avoid this mapping leaking objects that used to be replicated. */
		TMap<FConcertObjectInStreamID, FPlatformSecondsTimestamp> ObjectsToLastProcessed;
	};

	/** Subclass of FFrequencyTracker that you call Tick on. Every x seconds, this will call CleanseOutdatedObjects itself. */
	class FFrequencyTracker_CleanByTick : public FFrequencyTracker
	{
	public:

		FFrequencyTracker_CleanByTick(const double InCleanUpInterval = 5.0)
			: CleanUpIntervalSeconds(InCleanUpInterval)
		{}

		UE_API void Tick(float DeltaTime, const FPlatformSecondsTimestamp NowAsPlatformSeconds = FPlatformTime::Seconds());

	private:

		const double CleanUpIntervalSeconds;
		double SecondsSinceLastCleanse = 0.f;
	};
}

#undef UE_API
