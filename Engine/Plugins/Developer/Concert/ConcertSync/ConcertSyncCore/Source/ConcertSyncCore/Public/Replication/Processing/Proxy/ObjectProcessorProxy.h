// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Processing/ObjectReplicationProcessor.h"

#include <type_traits>

namespace UE::ConcertSyncCore
{
	template <typename T>
	concept TIsObjectProcessorConcept = std::is_base_of_v<FObjectReplicationProcessor, T>;
	
    /**
     * A proxy processor adds additional operations on top of FObjectReplicationProcessor at compile time.
     * The advantage of this approach is that multiple behaviours can flexibly be chained on top of a real processor implementation with minimal overhead.
     * 
     * This class is supposed to be subclassed and call functions on the real processor implementation.
     * The point of this class is to be a marker interface (i.e. when somebody reads TObjectProcessorProxy they know what the intention is).
     */
    template<TIsObjectProcessorConcept TRealProcessorImpl>
    class TObjectProcessorProxy : public TRealProcessorImpl
    {
    public:
    	
	    /**
		 * @param Arg Perfectly forwarded args for constructing the real processor implementation
	     */
	    template<typename... TArg>
    	TObjectProcessorProxy(TArg&&... Arg)
    		: TRealProcessorImpl(Forward<TArg>(Arg)...)
    	{}

    protected:

    	const TRealProcessorImpl& GetInnerProcessor() const { return *this; }
    	TRealProcessorImpl& GetInnerProcessor() { return *this; }
    };
}
