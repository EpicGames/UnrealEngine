// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrequencyTracker.h"
#include "ObjectProcessorProxy.h"
#include "Replication/Data/ReplicationFrequencySettings.h"

#include "Delegates/Delegate.h"
#include "HAL/PlatformTime.h"

namespace UE::ConcertSyncCore
{
	DECLARE_DELEGATE_RetVal_OneParam(FConcertObjectReplicationSettings, FGetObjectFrequencySettings, const FConcertReplicatedObjectId& Object);
	
	/**
	 * Maps every FSoftObjectPath to FConcertObjectReplicationSettings.
	 * Extends the inner IObjectReplicationProcessor by throttling the processing of objects according to the associated frequency settings.  
	 */
	template<TIsObjectProcessorConcept TRealProcessorImpl>
	class TObjectProcessorProxy_Frequency : public TObjectProcessorProxy<TRealProcessorImpl>
    {
	public:
		
		/**
		 * @param GetObjectFrequencySettingsDelegate Gets the frequency settings for an object in a stream
		 * @param Arg Perfectly forwarded args for constructing the real processor implementation
		 */
		template<typename... TArg>
		TObjectProcessorProxy_Frequency(FGetObjectFrequencySettings GetObjectFrequencySettingsDelegate, TArg&&... Arg)
			: TObjectProcessorProxy<TRealProcessorImpl>(Forward<TArg>(Arg)...)
			, GetObjectFrequencySettingsDelegate(MoveTemp(GetObjectFrequencySettingsDelegate))
		{}

		/**
		 * @param GetObjectFrequencySettingsDelegate Gets the frequency settings for an object in a stream
		 * @param ObjectCacheRetentionSeconds If an object goes this time without being processed, it is removed from the cache.
		 * @param Arg Perfectly forwarded args for constructing the real processor implementation
		 */
		template<typename... TArg>
		TObjectProcessorProxy_Frequency(FGetObjectFrequencySettings GetObjectFrequencySettingsDelegate, const float ObjectCacheRetentionSeconds, TArg&&... Arg)
			: TObjectProcessorProxy<TRealProcessorImpl>(Forward<TArg>(Arg)...)
			, GetObjectFrequencySettingsDelegate(MoveTemp(GetObjectFrequencySettingsDelegate))
			, FrequencyTracker(ObjectCacheRetentionSeconds)
		{}

		//~ Begin FObjectReplicationProcessor Interface
		virtual void ProcessObjects(const FProcessObjectsParams& Params) override;
		//~ End FObjectReplicationProcessor Interface

	protected:
		
		//~ Begin FObjectReplicationProcessor Interface
		virtual void ProcessObject(const FObjectReplicationProcessor::FObjectProcessArgs& Args) override;
		//~ End FObjectReplicationProcessor Interface

	private:

		/** Gets an objet's frequency settings */
		const FGetObjectFrequencySettings GetObjectFrequencySettingsDelegate;
		
		/**
		 * Used for throttling object replication.
		 * Tracks when an object is processed and can be queried whether it is time to process the object again.
		 */
		FFrequencyTracker_CleanByTick FrequencyTracker;

		/** Updated with every invocation of ProcessObjects and used by ProcessObject. The result of FPlatformTime::Seconds(). */
		double CurrentProcessingTime{};
    };

	template <TIsObjectProcessorConcept TRealProcessorImpl>
	void TObjectProcessorProxy_Frequency<TRealProcessorImpl>::ProcessObjects(const FProcessObjectsParams& Params)
	{
		CurrentProcessingTime = FPlatformTime::Seconds();
		// Normally calls ProcessObject on all objects that want to be replicated
		TRealProcessorImpl::ProcessObjects(Params);
		
		// This removes objects from the cache that have not been replicated in a while
		FrequencyTracker.Tick(Params.DeltaTime);
	}

	template <TIsObjectProcessorConcept TRealProcessorImpl>
	void TObjectProcessorProxy_Frequency<TRealProcessorImpl>::ProcessObject(const FObjectReplicationProcessor::FObjectProcessArgs& Args)
	{
		const FConcertReplicatedObjectId& Object = Args.ObjectInfo.ObjectId;
		const FConcertObjectReplicationSettings FrequencySettings = GetObjectFrequencySettingsDelegate.Execute(Object);

		// Throttle calls to the real ProcessObject implementation based on the object's frequency settings
		if (FrequencyTracker.TrackAndCheckObject(Object, FrequencySettings, CurrentProcessingTime) == ECanProcessObject::CanProcess)
		{
			FrequencyTracker.OnProcessObject(Object, CurrentProcessingTime);
			
			// Do the actual processing of the object
			TRealProcessorImpl::ProcessObject(Args);
		}
	}
}
