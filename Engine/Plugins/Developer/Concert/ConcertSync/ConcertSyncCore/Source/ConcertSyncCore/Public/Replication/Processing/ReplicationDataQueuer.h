// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IReplicationDataSource.h"
#include "ObjectReplicationCache.h"

#define UE_API CONCERTSYNCCORE_API

namespace UE::ConcertSyncCore
{
	/**
	 * Holds on to events received from remote endpoints and exposes them as a source.
	 * 
	 * Received events are received from a FObjectReplicationCache, which makes sure that event data is shared effectively
	 * if you created multiple FReplicationDataQueuer based on the same cache. This is relevant server side, where a
	 * FReplicationDataQueuer is created for each client.
	 * 
	 * Child classes need to implement IReplicationCacheUser::WantsToAcceptObject.
	 * TODO: Remove object data if the object is is meant for becomes unavailable.
	 */
	class FReplicationDataQueuer
		: public IReplicationDataSource
		, public IReplicationCacheUser
		, public TSharedFromThis<FReplicationDataQueuer>
	{
	public:

		//~ Begin IReplicationDataSource Interface
		UE_API virtual void ConsumePendingObjects(TFunctionRef<void(const FPendingObjectReplicationInfo& Info, IReplicationDataItem& Item)> InProcess) override;
		//~ End IReplicationDataSource Interface
		
		//~ Begin IReplicationCacheUser Interface
		UE_API virtual void OnDataCached(const FConcertReplicatedObjectId& Object, FSequenceId SequenceId, TSharedRef<const FConcertReplication_ObjectReplicationEvent> Data) override;
		UE_API virtual void OnCachedDataUpdated(const FConcertReplicatedObjectId& Object, FSequenceId SequenceId) override;
		//~ End IReplicationCacheUser Interface

	protected:

		/**
		 * Called by subclass factory functions.
		 * @param InReplicationCache The cache that the data is sent to. The caller ensures it outlives the lifetime of the widget.
		 */
		UE_API void BindToCache(FObjectReplicationCache& InReplicationCache);
		
	private:

		struct FPendingObjectData
		{
			/** Pointer to what OnDataCached passed to us. */
			TSharedPtr<const FConcertReplication_ObjectReplicationEvent> DataToApply;
			/** The latest SequenceId that the data contains. The data might contain data from multiple sequences. */
			FSequenceId SequenceId;
		};

		/** Stores events as they are received. */
		TMap<FConcertReplicatedObjectId, FPendingObjectData> PendingEvents;

		/** Provides us with replication events and shares them effectively. */
		FObjectReplicationCache* ReplicationCache = nullptr;
	};
}

#undef UE_API
