// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/ConcertPropertySelection.h"
#include "Misc/EBreakBehavior.h"
#include "Templates/Function.h"

class FProperty;
class UStruct;
struct FArchiveSerializedPropertyChain;

namespace UE::ConcertSyncCore::PropertyChain
{
	/** Tries to find the property referenced by the Chain. */
	CONCERTSYNCCORE_API FProperty* ResolveProperty(const UStruct& Class, const FConcertPropertyChain& ChainToResolve, bool bLogOnFail = true);

	/**
	 * Iterates all properties that are supported for replication.
	 * 
	 * This iterates every property as described by FConcertPropertyChain::PathToProperty, which means inner properties
	 * of containers, like TArray, TSet, and TMap, are skipped.
	 *
	 * @param Class The class whose properties to look at
	 * @param ProcessProperty Callback
	 * @param OnlyChildrenOf Instead of iterating all properties, you can restrict to only receive the child properties of OnlyChildrenOf.
	 */
	CONCERTSYNCCORE_API void ForEachReplicatableProperty(
		const UStruct& Class,
		TFunctionRef<EBreakBehavior(const FArchiveSerializedPropertyChain& Chain, FProperty& LeafProperty)> ProcessProperty
		);

	/**
	 * Iterates all properties that are supported for replication and converts them to FConcertPropertyChain.
	 *
	 * This iterates every property as described by FConcertPropertyChain::PathToProperty, which means inner properties
	 * of containers, like TArray, TSet, and TMap, are skipped.
	 *
	 * @param Class The class whose properties to look at
	 * @param ProcessProperty Callback
	 * @param OnlyChildrenOf Instead of iterating all properties, you can restrict to only receive the child properties of OnlyChildrenOf.
	 */
	CONCERTSYNCCORE_API void ForEachReplicatableConcertProperty(
		const UStruct& Class,
		TFunctionRef<EBreakBehavior(FConcertPropertyChain&& PropertyChain)> ProcessProperty
		);

	
	/**
	 * Efficiently create a batch of paths from the same class hierarchy.
	 *
	 * @param Class The class for which the paths are created
	 * @param NumPaths The number of paths we're looking to create. Once MatchesPath has returned true NumPaths times, this function terminates.
	 * @param MatchesPath Determines whether you are interested in this path. If so, do something with it and return true.
	 */
	CONCERTSYNCCORE_API void BulkConstructConcertChainsFromPaths(
		const UStruct& Class,
		uint32 NumPaths,
		TFunctionRef<bool(const FArchiveSerializedPropertyChain& Chain, FProperty& LeafProperty)> MatchesPath
		);

	/** Util that can be used with BulkConstructConcertChainsFromPaths. Checks whether Path coincides with Chain & LeafProperty.*/
	CONCERTSYNCCORE_API bool DoPathAndChainsMatch(const TArray<FName>& Path, const FArchiveSerializedPropertyChain& Chain, const FProperty& LeafProperty);


	/**
	 * Decides whether the property is allowed for replication.
	 * @param LeafProperty The property being considered.
	 */
	CONCERTSYNCCORE_API bool IsReplicatableProperty(const FProperty& LeafProperty);

	/** Util for determining whether this property is the inner property of a container. */
	CONCERTSYNCCORE_API bool IsInnerContainerProperty(const FProperty& Property);

	/**
	 * Util for determining whether this property is considered primitive (numeric, bool, enum).
	 * Primitive properties in containers are indicated as FConcertPropertyChain::InternalContainerPropertyValueName, which is "Value".
	 */
	CONCERTSYNCCORE_API bool IsPrimitiveProperty(const FProperty& Property);
	/**
	 * Util for determining whether this struct is considered native.
	 * Native struct properties in containers are indicated as FConcertPropertyChain::InternalContainerPropertyValueName, which is "Value".
	 */
	CONCERTSYNCCORE_API bool IsNativeStructProperty(const FProperty& Property);
}
