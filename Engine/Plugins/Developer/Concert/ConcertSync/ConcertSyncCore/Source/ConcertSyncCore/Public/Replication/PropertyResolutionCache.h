// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Data/ConcertPropertySelection.h"

#define UE_API CONCERTSYNCCORE_API

class UStruct;

namespace UE::ConcertSyncCore::PropertyChain
{
	/**
	 * FConcertPropertyChain::ResolveProperty is expensive because it has to iterate the property hierarchy.
	 * 
	 * This caches look-up results of native classes.
	 * By default, properties in Blueprint classes are not cached because they can change at runtime and leave dangling FProperty pointers.
	 */
	class FPropertyResolutionCache
	{
	public:

		/** Resolves the property chain and caches the result; */
		UE_API FProperty* ResolveAndCache(const UStruct& Struct, const FConcertPropertyChain& Chain);

		/** Removes the class from the cache.*/
		void Invalidate(const UStruct& Struct) { CachedProperties.Remove(&Struct); }
		/** Removes everything from the cache. */
		void Clear(int32 ExpectedNumElements = 0) { CachedProperties.Empty(ExpectedNumElements); }

	private:

		struct FClassCache
		{
			TMap<FConcertPropertyChain, FProperty*> Cache;
		};

		TMap<const UStruct*, FClassCache> CachedProperties;
	};
}

#undef UE_API
