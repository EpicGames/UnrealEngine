// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

#define UE_API CONCERTSYNCCORE_API

class FProperty;
struct FArchiveSerializedPropertyChain;
struct FConcertPropertyChain;
struct FConcertPropertySelection;

namespace UE::ConcertSyncCore
{
	/**
	 * Util that bridges IObjectReplicationFormat::FAllowPropertyFunc and FConcertPropertySelection.
	 *
	 * Intended to be used like this:
	 * {
	 *		const FConcertPropertySelection& Selection = ...;
	 *		IObjectReplicationFormat& Format = ...;
	 *		UObject& ReplicatedObject = ...;
	 *		
	 *		FReplicationPropertyFilter Filter(Selection);
	 *		Format.CreateReplicationEvent(Object, [&Filter](const FArchiveSerializedPropertyChain* Chain, const FProperty&& Property){ return Filter.IsPropertyInSelection(Chain, Property); }
	 *	}
	 */
	class FReplicationPropertyFilter
	{
		const FConcertPropertySelection& PropertySelection;
		
		/**
		 * Maps the name of every leaf property to the chaisn PropertySelection.ReplicatedProperties that contain the property name at the end of the chain.
		 * This speeds up matching chain and property.
		 */
		TMap<FName, TArray<const FConcertPropertyChain*>> LeafToChain;
	public:

		UE_API FReplicationPropertyFilter(const FConcertPropertySelection& PropertySelection);
		
		/**
		 * @param Chain Can be nullptr or non-null and empty - both imply the Property is a root property.
		 * @param Property The property that Chain leads to
		 * @return Whether Property is in PropertySelection.
		 */
		UE_API bool ShouldSerializeProperty(const FArchiveSerializedPropertyChain* Chain, const FProperty& Property) const;
	};
}

#undef UE_API
