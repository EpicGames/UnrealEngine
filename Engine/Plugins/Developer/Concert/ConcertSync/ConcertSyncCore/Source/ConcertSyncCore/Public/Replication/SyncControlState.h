// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/ObjectIds.h"
#include "Messages/SyncControl.h"
#include "Misc/EBreakBehavior.h"
#include "Replication/Messages/ChangeAuthority.h"
#include "Replication/Messages/ChangeStream.h"
#include "Replication/Messages/SyncControl.h"

#include "Templates/Function.h"

#include <type_traits>

#include "Messages/Muting.h"
#include "Messages/RestoreContent.h"

namespace UE::ConcertSyncCore::Replication
{
	template<typename T>
	concept CObjectInStreamInvocable = std::is_invocable_v<T, const FConcertObjectInStreamID&>;
	
	/**
	 * Keeps track of the sync control (AllowedObjects), which is a set of objects a client is allowed to replicate.
	 * It knows how to parse network messages that explicitly and implicitly change sync control and update AllowedObjects.
	 * 
	 * This class does not know where the network messages come from and is designed to be used as utility by lower level systems.
	 */
	class FSyncControlState
	{
	public:

		class FPredictedObjectRemoval
		{
			friend class FSyncControlState;
			TSet<FConcertObjectInStreamID> Objects;
		};

		FSyncControlState() = default;
		
		// Not explicit to make it useful to use
		FSyncControlState(TSet<FConcertObjectInStreamID>&& AllowedObjects) 
			: AllowedObjects(MoveTemp(AllowedObjects))
		{}
		FSyncControlState(const TSet<FConcertObjectInStreamID>& AllowedObjects)
			: AllowedObjects(AllowedObjects)
		{}

		/** @return Whether Object is allowed to be replicated. */
		bool IsObjectAllowed(const FConcertObjectInStreamID& Object) const { return AllowedObjects.Contains(Object); }
		/** @return Number of allowed objects */
		uint32 Num() const { return AllowedObjects.Num(); }

		/** Enumerates all sync controlled objects. */
		template<typename TProcessObject>
		bool EnumerateAllowedObjects(TProcessObject Callback) const requires std::is_invocable_r_v<EBreakBehavior, TProcessObject, const FConcertObjectInStreamID&>;

		/** Enumerates the changes that would be made by applying this event. */
		template<CObjectInStreamInvocable TOnAllowed, CObjectInStreamInvocable TOnDisallowed>
		static void EnumerateChanges(const FConcertReplication_ChangeSyncControl& Event, TOnAllowed&& OnAllowed, TOnDisallowed&& OnDisallowed);
		
		/** Combines past state with an incoming sync control event. */
		template<CObjectInStreamInvocable TOnAllowed, CObjectInStreamInvocable TOnDisallowed>
		void AppendChanges(const FConcertReplication_ChangeSyncControl& Event, TOnAllowed&& OnAllowed, TOnDisallowed&& OnDisallowed);
		void AppendChanges(const FConcertReplication_ChangeSyncControl& Event);

		/** Combines implicit and explicit sync control changes caused by authority changes. */
		template<CObjectInStreamInvocable TOnAllowed, CObjectInStreamInvocable TOnDisallowed>
		void AppendAuthorityChange(const FConcertReplication_ChangeAuthority_Request& Request, const FConcertReplication_ChangeSyncControl& Response, TOnAllowed&& OnAllowed, TOnDisallowed&& OnDisallowed);
		void AppendAuthorityChange(const FConcertReplication_ChangeAuthority_Request& Request, const FConcertReplication_ChangeSyncControl& Response);

		/** Combines implicit sync control changes caused by stream change. */
		template<CObjectInStreamInvocable TOnDisallowed>
		void AppendStreamChange(const FConcertReplication_ChangeStream_Request& Request, TOnDisallowed&& OnDisallowed);
		void AppendStreamChange(const FConcertReplication_ChangeStream_Request& Request);
		
		/**
		 * Combines implicit sync control changes caused by the local client sending a mute event.
		 * @return The objects that were predictively removed by the request. Pass this back into AppendChanges once you get the response.
		 */
		template<CObjectInStreamInvocable TOnDisallowed>
		FPredictedObjectRemoval PredictAndApplyMuteChanges(const FConcertReplication_ChangeMuteState_Request& Request, TOnDisallowed&& OnDisallowed);
		FPredictedObjectRemoval PredictAndApplyMuteChanges(const FConcertReplication_ChangeMuteState_Request& Request);
		/**
		 * Looks at the response:
		 * - if the change failed, reverts the predictively removed sync control
		 * - if the change succeeded, appends the contained sync control
		 */
		template<CObjectInStreamInvocable TOnAllowed>
		void ApplyOrRevertMuteResponse(const FPredictedObjectRemoval& ObjectsRemovedInRequest, const FConcertReplication_ChangeMuteState_Response& Response, TOnAllowed&& OnAllowed);
		void ApplyOrRevertMuteResponse(const FPredictedObjectRemoval& ObjectsRemovedInRequest, const FConcertReplication_ChangeMuteState_Response& Response);
		
		/**
		 * If the request replaces the content, removes sync control for all objects.
		 * @return The objects that were predictively removed by the request. Pass this back into AppendChanges once you get the response.
		 */
		template<CObjectInStreamInvocable TOnDisallowed>
		FPredictedObjectRemoval PredictAndApplyRestoreContentChanges(const FConcertReplication_RestoreContent_Request& Request, TOnDisallowed&& OnDisallowed);
		/**
		 * Looks at the response
		 * - if the change failed, reverts the predictively removed sync control.
		 * - if the change succeeded, appends the contained sync control.
		 */
		template<CObjectInStreamInvocable TOnAllowed, CObjectInStreamInvocable TOnDisallowed>
		void ApplyOrRevertRestoreContentResponse(const FPredictedObjectRemoval& ObjectsRemovedInRequest, const FConcertReplication_RestoreContent_Response& Response, TOnAllowed&& OnAllowed, TOnDisallowed&& OnDisallowed);
		
		friend bool operator==(const FSyncControlState& Left, const FSyncControlState& Right)
		{
			return Left.AllowedObjects.Num() == Right.AllowedObjects.Num() && Left.AllowedObjects.Includes(Right.AllowedObjects);
		}
		friend bool operator!=(const FSyncControlState& Left, const FSyncControlState& Right) { return !(Left == Right);}

	private:

		/** The objects the client has sync control over, i.e. is allowed to replicate. */
		TSet<FConcertObjectInStreamID> AllowedObjects;
	};
	
	template<typename TProcessObject>
	bool FSyncControlState::EnumerateAllowedObjects(TProcessObject Callback) const requires std::is_invocable_r_v<EBreakBehavior, TProcessObject, const FConcertObjectInStreamID&>
	{
		for (const FConcertObjectInStreamID& Object : AllowedObjects)
		{
			if (Callback(Object) == EBreakBehavior::Break)
			{
				break;
			}
		}

		return AllowedObjects.Num() > 0;
	}

	template <CObjectInStreamInvocable TOnAllowed, CObjectInStreamInvocable TOnDisallowed>
	void FSyncControlState::EnumerateChanges(const FConcertReplication_ChangeSyncControl& Event, TOnAllowed&& OnAllowed, TOnDisallowed&& OnDisallowed)
	{
		for (const TPair<FConcertObjectInStreamID, bool>& NewControlState : Event.NewControlStates)
		{
			const FConcertObjectInStreamID& Object = NewControlState.Key;
			if (NewControlState.Value)
			{
				OnAllowed(Object);
			}
			else
			{
				OnDisallowed(Object);
			}
		}
	}

	template<CObjectInStreamInvocable TOnAllowed, CObjectInStreamInvocable TOnDisallowed>
	void FSyncControlState::AppendChanges(const FConcertReplication_ChangeSyncControl& Event, TOnAllowed&& OnAllowed, TOnDisallowed&& OnDisallowed)
	{
		EnumerateChanges(Event,
			[this, &OnAllowed](const FConcertObjectInStreamID& Object){ AllowedObjects.Add(Object); OnAllowed(Object); },
			[this, &OnDisallowed](const FConcertObjectInStreamID& Object){ AllowedObjects.Remove(Object); OnDisallowed(Object); }
			);
	}

	inline void FSyncControlState::AppendChanges(const FConcertReplication_ChangeSyncControl& Event)
	{
		AppendChanges(Event, [](const FConcertObjectInStreamID&){}, [](const FConcertObjectInStreamID&){});
	}
	
	template <CObjectInStreamInvocable TOnAllowed, CObjectInStreamInvocable TOnDisallowed>
	void FSyncControlState::AppendAuthorityChange(const FConcertReplication_ChangeAuthority_Request& Request, const FConcertReplication_ChangeSyncControl& Response, TOnAllowed&& OnAllowed, TOnDisallowed&& OnDisallowed)
	{
		for (const TPair<FSoftObjectPath, FConcertStreamArray>& ImplicitChange : Request.ReleaseAuthority)
		{
			for (const FGuid& StreamId : ImplicitChange.Value.StreamIds)
			{
				const FConcertObjectInStreamID ImplicitlyRemoved { StreamId, ImplicitChange.Key };

				const int32 NumRemoved = AllowedObjects.Remove(ImplicitlyRemoved);
				if (NumRemoved > 0)
				{
					OnDisallowed(ImplicitlyRemoved);
				}
			}
		}

		AppendChanges(Response, MoveTemp(OnAllowed), MoveTemp(OnDisallowed));
	}

	inline void FSyncControlState::AppendAuthorityChange(const FConcertReplication_ChangeAuthority_Request& Request, const FConcertReplication_ChangeSyncControl& Response)
	{
		AppendAuthorityChange(Request, Response, [](const FConcertObjectInStreamID&){}, [](const FConcertObjectInStreamID&){});
	}
	
	template <CObjectInStreamInvocable TOnDisallowed>
	void FSyncControlState::AppendStreamChange(const FConcertReplication_ChangeStream_Request& Request, TOnDisallowed&& OnDisallowed)
	{
		for (auto It = AllowedObjects.CreateIterator(); It; ++It)
		{
			if (Request.StreamsToRemove.Contains(It->StreamId))
			{
				const FConcertObjectInStreamID StreamId = *It;
				It.RemoveCurrent();
				OnDisallowed(StreamId);
			}
		}
		
		if (AllowedObjects.IsEmpty())
		{
			return;
		}

		for (const FConcertObjectInStreamID& RemovedObject : Request.ObjectsToRemove)
		{
			if (IsObjectAllowed(RemovedObject))
			{
				AllowedObjects.Remove(RemovedObject);
				OnDisallowed(RemovedObject);

				if (AllowedObjects.IsEmpty())
				{
					break;
				}
			}
		}
	}

	inline void FSyncControlState::AppendStreamChange(const FConcertReplication_ChangeStream_Request& Request)
	{
		AppendStreamChange(Request, [](const FConcertObjectInStreamID&){});
	}

	template <CObjectInStreamInvocable TOnDisallowed>
	FSyncControlState::FPredictedObjectRemoval FSyncControlState::PredictAndApplyMuteChanges(const FConcertReplication_ChangeMuteState_Request& Request, TOnDisallowed&& OnDisallowed)
	{
		FPredictedObjectRemoval RemovedObjects;
		const auto Remove = [&OnDisallowed, &RemovedObjects](TSet<FConcertObjectInStreamID>::TIterator& It)
		{
			const FConcertObjectInStreamID Object = *It;
			It.RemoveCurrent();
						
			RemovedObjects.Objects.Add(Object);
			OnDisallowed(Object);
		};
		
		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& ObjectInfo : Request.ObjectsToMute)
		{
			const FSoftObjectPath& MutedObject = ObjectInfo.Key;
			const EConcertReplicationMuteOption MuteOption = ObjectInfo.Value.Flags;
			
			if (AffectSubobjects(MuteOption))
			{
				const FString MutedAsString = MutedObject.ToString();
				for (auto It = AllowedObjects.CreateIterator(); It; ++It)
				{
					const bool bRemove = It->Object.ToString().Contains(MutedObject.ToString());
					if (bRemove)
					{
						Remove(It);
					}
				}
			}
			else
			{
				for (auto It = AllowedObjects.CreateIterator(); It; ++It)
				{
					if (It->Object == MutedObject)
					{
						Remove(It);
					}
				}
			}
		}
		return RemovedObjects;
	}

	inline FSyncControlState::FPredictedObjectRemoval FSyncControlState::PredictAndApplyMuteChanges(const FConcertReplication_ChangeMuteState_Request& Request)
	{
		return PredictAndApplyMuteChanges(Request, [](const FConcertObjectInStreamID&){});
	}

	template <CObjectInStreamInvocable TOnAllowed>
	void FSyncControlState::ApplyOrRevertMuteResponse(const FPredictedObjectRemoval& ObjectsRemovedInRequest, const FConcertReplication_ChangeMuteState_Response& Response, TOnAllowed&& OnAllowed)
	{
		if (Response.IsSuccess())
		{
			AppendChanges(Response.SyncControl,
				OnAllowed,
				[](const FConcertObjectInStreamID&)
				{
					ensureAlwaysMsgf(false, TEXT("By contract, objects losing sync control are not supposed to be listed here. FConcertReplication_ChangeMuteState_Response::SyncControl documentation."));
				});
		}
		else
		{
			AllowedObjects.Append(ObjectsRemovedInRequest.Objects);
		}
	}

	inline void FSyncControlState::ApplyOrRevertMuteResponse(const FPredictedObjectRemoval& ObjectsRemovedInRequest, const FConcertReplication_ChangeMuteState_Response& Response)
	{
		ApplyOrRevertMuteResponse(ObjectsRemovedInRequest, Response, [](const FConcertObjectInStreamID&){});
	}

	template <CObjectInStreamInvocable TOnDisallowed>
	FSyncControlState::FPredictedObjectRemoval FSyncControlState::PredictAndApplyRestoreContentChanges(
		const FConcertReplication_RestoreContent_Request& Request,
		TOnDisallowed&& OnDisallowed
		)
	{
		FPredictedObjectRemoval PredictedRemoval;
		if (EnumHasAnyFlags(Request.Flags, EConcertReplicationRestoreContentFlags::RestoreOnTop))
		{
			return PredictedRemoval;
		}

		// The request will replace everything we have - so disallow all. 
		for (const FConcertObjectInStreamID& Object : AllowedObjects)
		{
			OnDisallowed(Object);
		}
		PredictedRemoval.Objects = MoveTemp(AllowedObjects);
		
		return PredictedRemoval;
	}

	template <CObjectInStreamInvocable TOnAllowed, CObjectInStreamInvocable TOnDisallowed>
	void FSyncControlState::ApplyOrRevertRestoreContentResponse(
		const FPredictedObjectRemoval& ObjectsRemovedInRequest,
		const FConcertReplication_RestoreContent_Response& Response,
		TOnAllowed&& OnAllowed,
		TOnDisallowed&& OnDisallowed
		)
	{
		if (Response.IsSuccess())
		{
			AppendChanges(Response.SyncControl, OnAllowed, OnDisallowed);
		}
		else
		{
			// Do not assign MoveTemp(ObjectsRemovedInRequest.Objects) because we have been modified between!
			AllowedObjects.Append(ObjectsRemovedInRequest.Objects);
		}
	}
}

